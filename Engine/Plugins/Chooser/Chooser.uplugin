{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Chooser",
	"Description" : "Use Chooser and Proxy Tables to build dynamic asset selection logic.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		},
		{
			"Name": "BlendStack",
			"Enabled": true
		},
		{
			"Name": "GameplayInsights",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "Chooser",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name": "ChooserUncooked",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name" : "ChooserEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "ProxyTable",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name": "ProxyTableUncooked",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "ProxyTableEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}
