// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Chooser : ModuleRules
	{
		public Chooser(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"AnimationCore",
					"AnimGraphRuntime",
					"BlendStack",
					"TraceLog",
					"RewindDebuggerRuntimeInterface",
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}