// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNode_RelevantAssetPlayerBase.h"
#include "Animation/AnimationAsset.h"
#include "CoreMinimal.h"
#include "IHasContext.h"
#include "StructUtils/InstancedStruct.h"
#include "IObjectChooser.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "ChooserTypes.h"
#include "AnimNode_ChooserPlayer.generated.h"

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_ChooserPlayer : public FAnimNode_BlendStack_Standalone
#if CPP
	, public IHasContextClass
#endif
{
	GENERATED_BODY()

	CHOOSER_API FAnimNode_ChooserPlayer();

public:
	virtual TConstArrayView<FInstancedStruct> GetContextData() const override { return ChooserContextDefinition; }
	
	// How often the chooser should be evaluated
	UPROPERTY(EditAnywhere, Category = "Chooser")
	EChooserEvaluationFrequency EvaluationFrequency = EChooserEvaluationFrequency::OnBecomeRelevant;
	
	// Type of chooser logic to use: Use "Evaluate Chooser" for chooser table evaluation, and "Lookup Proxy" for proxy table lookups
	UPROPERTY(EditAnywhere, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ObjectChooserBase"), Category = "Chooser")
	FInstancedStruct Chooser;

	// if set and bMirrored MirrorDataTable will be used for mirroring the aniamtion
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UMirrorDataTable> MirrorDataTable;
	
	// requested blend space blend X parameter (if AnimationAsset is a blend space)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float BlendSpaceX = 0;
	
	// requested blend space blend Y parameter (if AnimationAsset is a blend space)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	float BlendSpaceY = 0;

	// Should BlendSpaceParameters be applied to all blend spaces, including ones that are in the process of blending out.
	bool bUpdateAllActiveBlendSpaces = true;
	
	// Settings when starting an animation - these can be overridden per animation asset by the chooser itself
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PinHiddenByDefault))
	FChooserPlayerSettings DefaultSettings;
	
	UPROPERTY()
	TArray<FInstancedStruct> ChooserContextDefinition;

	// Use pose matching to choose the start position. Requires PoseSearch plugin.
	UPROPERTY(EditAnywhere, Category = PoseMatching, meta = (PinHiddenByDefault))
	bool bStartFromMatchingPose = false;

	// FAnimNode_Base interface
	CHOOSER_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	CHOOSER_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	// End of FAnimNode_Base interface
	
	// FAnimNode_AssetPlayerBase interface
	CHOOSER_API virtual void UpdateAssetPlayer(const FAnimationUpdateContext& Context) override;
	CHOOSER_API virtual FName GetGroupName() const override;
	CHOOSER_API virtual EAnimGroupRole::Type GetGroupRole() const override;
	CHOOSER_API virtual EAnimSyncMethod GetGroupMethod() const override;
	CHOOSER_API virtual bool GetIgnoreForRelevancyTest() const override;
	CHOOSER_API virtual bool IsLooping() const override;
	CHOOSER_API virtual bool SetGroupName(FName InGroupName) override;
	CHOOSER_API virtual bool SetGroupRole(EAnimGroupRole::Type InRole) override;
	CHOOSER_API virtual bool SetGroupMethod(EAnimSyncMethod InMethod) override;
	CHOOSER_API virtual bool SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest) override;
	// End of FAnimNode_AssetPlayerBase interface
	
private:
	UAnimationAsset* ChooseAsset(const FAnimationUpdateContext& Context);
	FChooserPlayerSettings Settings;
	UAnimationAsset* CurrentAsset = nullptr;
	float CurrentStartTime = 0;
	bool CurrentMirror = false;
	bool bInitialized = false;
	uint32 CurrentCurveOverridesHash = 0;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;
	
#if WITH_EDITORONLY_DATA
	// The group name that we synchronize with (NAME_None if it is not part of any group). Note that
	// this is the name of the group used to sync the output of this node - it will not force
	// syncing of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	FName GroupName = NAME_None;

	// The role this node can assume within the group (ignored if GroupName is not set). Note
	// that this is the role of the output of this node, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	TEnumAsByte<EAnimGroupRole::Type> GroupRole = EAnimGroupRole::CanBeLeader;

	// How this node will synchronize with other animations. Note that this determines how the output
	// of this node is used for synchronization, not of animations contained by it.
	UPROPERTY(EditAnywhere, Category = Sync, meta = (FoldProperty))
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;

	// If true, "Relevant anim" nodes that look for the highest weighted animation in a state will ignore this node
	UPROPERTY(EditAnywhere, Category = Relevancy, meta = (FoldProperty, PinHiddenByDefault))
	bool bIgnoreForRelevancyTest = false;
#endif // WITH_EDITORONLY_DATA
};
