// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_ChooserPlayer.h"

#include "IHasContext.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTrace.h"
#include "Animation/BlendSpace.h"
#include "BlendStack/AnimNode_BlendStack.h"
#include "IObjectChooser.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ChooserPlayer)

uint32 GetTypeHash(const FAnimCurveOverride& CurveOverride)
{
	return HashCombine(GetTypeHash(CurveOverride.CurveName.ToString()), GetTypeHash(CurveOverride.CurveValue));
}

void FAnimCurveOverrideList::ComputeHash()
{
	if (Values.Num() == 0)
	{
		Hash = 0;
	}
	else
	{
		Hash = GetArrayHash(&Values[0], Values.Num());
	}
}

FAnimNode_ChooserPlayer::FAnimNode_ChooserPlayer()
{
}

UAnimationAsset* FAnimNode_ChooserPlayer::ChooseAsset(const FAnimationUpdateContext& Context)
{
	using namespace UE::Anim;

	if (Chooser.IsValid())
	{
		// reset settings to default
		Settings = DefaultSettings;
		
		Settings.PlayingAsset = GetAnimAsset();
		Settings.PlayingAssetAccumulatedTime = GetAccumulatedTime();
		Settings.bIsPlayingAssetMirrored = GetMirror();
		Settings.PlayingAssetBlendParameters = GetBlendParameters();
		Settings.AnimationUpdateContext = &Context;

		FChooserEvaluationContext ChooserContext;
		ChooserContext.AddObjectParam(Context.GetAnimInstanceObject());
		ChooserContext.AddStructParam(Settings);

		const FObjectChooserBase& ChooserBase = Chooser.Get<FObjectChooserBase>();
		if (bStartFromMatchingPose)
		{
			if (IPoseSearchProvider* PoseSearchProvider = IPoseSearchProvider::Get())
			{
				TArray<const UObject*, TInlineAllocator<128>> ChosenAssets;
				ChooserBase.ChooseMulti(ChooserContext, FObjectChooserBase::FObjectChooserIteratorCallback::CreateLambda([&ChosenAssets](UObject* InResult)
					{
						ChosenAssets.Add(InResult);
						return FObjectChooserBase::EIteratorStatus::Continue;
					}));

				IPoseSearchProvider::FSearchPlayingAsset PlayingAsset;
				PlayingAsset.Asset = GetAnimAsset();
				PlayingAsset.AccumulatedTime = GetAccumulatedTime();
				PlayingAsset.bMirrored = GetMirror();
				PlayingAsset.BlendParameters = GetBlendParameters();

				const IPoseSearchProvider::FSearchResult SearchResult = PoseSearchProvider->Search(Context, ChosenAssets, PlayingAsset, IPoseSearchProvider::FSearchFutureAsset() );
				if (UAnimationAsset* SelectedAnimationAsset = Cast<UAnimationAsset>(SearchResult.SelectedAsset))
				{
					if (!SearchResult.bIsFromContinuingPlaying)
					{
						Settings.bForceBlendTo = true;
						Settings.StartTime = SearchResult.TimeOffsetSeconds;
						Settings.bMirror = SearchResult.bMirrored;
					}
					return SelectedAnimationAsset;
				}
			}
		}
		
		return Cast<UAnimationAsset>(ChooserBase.ChooseObject(ChooserContext));
	}
	return nullptr;
}

void FAnimNode_ChooserPlayer::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_BlendStack_Standalone::Initialize_AnyThread(Context);
	
	if(!bInitialized)
	{
		if (FObjectChooserBase* ObjectChooser = Chooser.GetMutablePtr<FObjectChooserBase>())
		{
			ObjectChooser->Compile(this,false);
		}
		
		bInitialized = true;
	}
}


void FAnimNode_ChooserPlayer::UpdateAssetPlayer(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateAassetPlayer)
	GetEvaluateGraphExposedInputs().Execute(Context);

	if (!Chooser.IsValid())
	{
		// We don't have any entries, play data will be invalid - early out
		return;
	}
	
	const bool bJustBecameRelevant = !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter());
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	if (CurrentAsset != nullptr && AnimPlayers.IsEmpty())
	{
		// catch the case where the blendstack has been reset, and we need to reinitialize it.
		if (EvaluationFrequency != EChooserEvaluationFrequency::OnInitialUpdate)
		{
			// force reselection for all modes other than OnInitialUpdate (in that case we will just restart the current animation below)
			CurrentAsset = nullptr;
		}

	}

	UAnimationAsset* NewAsset = CurrentAsset;

	if (EvaluationFrequency == EChooserEvaluationFrequency::OnUpdate || CurrentAsset == nullptr)
	{
		NewAsset = ChooseAsset(Context);
	}
	else if (EvaluationFrequency == EChooserEvaluationFrequency::OnLoop)
	{
		if (!AnimPlayers.IsEmpty())
		{
			FBlendStackAnimPlayer& Player = AnimPlayers[0];
			if (Player.GetPlayRate() * Context.GetDeltaTime() + Player.GetAccumulatedTime() > CurrentAsset->GetPlayLength())
			{
				NewAsset = ChooseAsset(Context);
			}
		}
	}
	else if (EvaluationFrequency == EChooserEvaluationFrequency::OnBecomeRelevant && bJustBecameRelevant)
	{
		NewAsset = ChooseAsset(Context);
	}

	// Restart the animation:
	// - if we've been told to do so via bForceBlendTo
	// - if this node just became relevant
	// - if we chose a new animation
	// - if the mirror setting has changed
	// - for playback rate of 0, when the start time changes (within Settings.MinDeltaTimeToForceBlendTo seconds of tolerance) - for choosing poses as frames of an animation sequence - @todo: handle looping properly
	// - if the curve values are different
	if (Settings.bForceBlendTo || bJustBecameRelevant || NewAsset != CurrentAsset || AnimPlayers.IsEmpty() ||
		CurrentMirror != Settings.bMirror ||
		(!FMath::IsNearlyEqual(CurrentStartTime, Settings.StartTime, Settings.MinDeltaTimeToForceBlendTo) && Settings.PlaybackRate == 0.0f) ||
		CurrentCurveOverridesHash != Settings.CurveOverrides.Hash)
	{
		Settings.bForceBlendTo = false;
		CurrentCurveOverridesHash = Settings.CurveOverrides.Hash;
		CurrentMirror = Settings.bMirror;
		
		float BlendTime = bJustBecameRelevant ? 0 : Settings.BlendTime;

		bool bLoop = Settings.bForceLooping;
		if (!bLoop)
		{
			if (const UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(NewAsset))
			{
				bLoop = Sequence->bLoop;				
			}
			else if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(NewAsset))
			{
				bLoop = BlendSpace->bLoop;
			}
		}
		
		FAnimNode_BlendStack_Standalone::BlendTo(Context, NewAsset, Settings.StartTime,
			bLoop, Settings.bMirror, MirrorDataTable, Settings.BlendTime,
			Settings.BlendProfile, Settings.BlendOption, Settings.bUseInertialBlend, NAME_None, FVector::Zero(), Settings.PlaybackRate, 0.f,
			GetGroupName(), GetGroupRole(), GetGroupMethod());

		if (!Settings.CurveOverrides.Values.IsEmpty())
		{
			TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElement>& OverrideCurve = AnimPlayers[0].OverrideCurve;
			OverrideCurve.Reserve(Settings.CurveOverrides.Values.Num());
			for (const FAnimCurveOverride& CurveOverride : Settings.CurveOverrides.Values)
			{
				OverrideCurve.Add(CurveOverride.CurveName, CurveOverride.CurveValue);
			}
		}
		
		CurrentAsset = NewAsset;
		CurrentStartTime = Settings.StartTime;
	}

	// Update blend space parameters
	const FVector BlendParameters(BlendSpaceX, BlendSpaceY, 0);
	const EBlendStack_BlendspaceUpdateMode BlendspaceUpdateMode = bUpdateAllActiveBlendSpaces ? 
																EBlendStack_BlendspaceUpdateMode::UpdateAll :
																EBlendStack_BlendspaceUpdateMode::UpdateActiveOnly;

	UpdateBlendspaceParameters(BlendspaceUpdateMode, BlendParameters);

	FAnimNode_BlendStack_Standalone::UpdateAssetPlayer(Context);

	FString Name;
	Chooser.Get<FObjectChooserBase>().GetDebugName(Name);
	
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), ToCStr(Name));
}

void FAnimNode_ChooserPlayer::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread);
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(MotionMatching, !IsInGameThread());

	FAnimNode_BlendStack_Standalone::Evaluate_AnyThread(Output);
}

FName FAnimNode_ChooserPlayer::GetGroupName() const
{
	return GET_ANIM_NODE_DATA(FName, GroupName);
}

EAnimGroupRole::Type FAnimNode_ChooserPlayer::GetGroupRole() const
{
	return GET_ANIM_NODE_DATA(TEnumAsByte<EAnimGroupRole::Type>, GroupRole);
}

EAnimSyncMethod FAnimNode_ChooserPlayer::GetGroupMethod() const
{
	return GET_ANIM_NODE_DATA(EAnimSyncMethod, Method);
}

bool FAnimNode_ChooserPlayer::GetIgnoreForRelevancyTest() const
{
	return GET_ANIM_NODE_DATA(bool, bIgnoreForRelevancyTest);
}

bool FAnimNode_ChooserPlayer::SetGroupName(FName InGroupName)
{
#if WITH_EDITORONLY_DATA
	GroupName = InGroupName;
#endif

	if(FName* GroupNamePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(FName, GroupName))
	{
		*GroupNamePtr = InGroupName;
		return true;
	}

	return false;
}

bool FAnimNode_ChooserPlayer::SetGroupRole(EAnimGroupRole::Type InRole)
{
#if WITH_EDITORONLY_DATA
	GroupRole = InRole;
#endif
	
	if(TEnumAsByte<EAnimGroupRole::Type>* GroupRolePtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(TEnumAsByte<EAnimGroupRole::Type>, GroupRole))
	{
		*GroupRolePtr = InRole;
		return true;
	}

	return false;
}

bool FAnimNode_ChooserPlayer::SetGroupMethod(EAnimSyncMethod InMethod)
{
#if WITH_EDITORONLY_DATA
	Method = InMethod;
#endif

	if(EAnimSyncMethod* MethodPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(EAnimSyncMethod, Method))
	{
		*MethodPtr = InMethod;
		return true;
	}

	return false;
}

bool FAnimNode_ChooserPlayer::IsLooping() const
{
	if (!AnimPlayers.IsEmpty())
	{
		return AnimPlayers[0].IsLooping();
	}
	return false;
}


bool FAnimNode_ChooserPlayer::SetIgnoreForRelevancyTest(bool bInIgnoreForRelevancyTest)
{
#if WITH_EDITORONLY_DATA
	bIgnoreForRelevancyTest = bInIgnoreForRelevancyTest;
#endif

	if (bool* bIgnoreForRelevancyTestPtr = GET_INSTANCE_ANIM_NODE_DATA_PTR(bool, bIgnoreForRelevancyTest))
	{
		*bIgnoreForRelevancyTestPtr = bInIgnoreForRelevancyTest;
		return true;
	}

	return false;
}