// Copyright Epic Games, Inc. All Rights Reserved.
#include "BoolColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "Chooser.h"
#include "ChooserTrace.h"

#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoolColumn)

bool FBoolContextProperty::GetValue(FChooserEvaluationContext& Context, bool& OutResult) const
{
	return Binding.GetValue(Context, OutResult);
}

bool FBoolContextProperty::SetValue(FChooserEvaluationContext& Context, bool InValue) const
{
	return Binding.SetValue(Context, InValue);
}

FBoolColumn::FBoolColumn()
{
	InputValue.InitializeAs(FBoolContextProperty::StaticStruct());
}

void FBoolColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	bool Result = false;
	if (InputValue.IsValid() && 
		InputValue.Get<FChooserParameterBoolBase>().GetValue(Context, Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result);

	#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
	#endif
		
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValuesWithAny.IsValidIndex(IndexData.Index))
			{
				
				if (RowValuesWithAny[IndexData.Index] == EBoolColumnCellValue::MatchAny || Result == static_cast<bool>(RowValuesWithAny[IndexData.Index]))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FBoolColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBoolBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Enum, StaticEnum<EBoolColumnCellValue>());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueEnum(PropertyName, RowValuesWithAny[RowIndex]);
	}

	void FBoolColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<uint8, EPropertyBagResult> Result = PropertyBag.GetValueEnum(PropertyName, StaticEnum<EBoolColumnCellValue>());
		if (uint8* Value = Result.TryGetValue())
		{
			RowValuesWithAny[RowIndex] = static_cast<EBoolColumnCellValue>(*Value);
		}
	}
#endif
