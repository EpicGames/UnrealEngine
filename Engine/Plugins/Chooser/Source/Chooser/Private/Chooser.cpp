// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chooser.h"
#include "ChooserFunctionLibrary.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#include "ObjectTrace.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Engine/Blueprint.h"
#include "ChooserIndexArray.h"
#include "IChooserParameterGameplayTag.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITORONLY_DATA
#include "UObject/UObjectIterator.h"
#endif

#include "OutputObjectColumn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Chooser)

DEFINE_LOG_CATEGORY(LogChooser)


#if WITH_EDITOR
const FName UChooserTable::PropertyNamesTag = "ChooserPropertyNames";
const FString UChooserTable::PropertyTagDelimiter = TEXT(";");
#endif

UChooserTable::UChooserTable(const FObjectInitializer& Initializer)
	:Super(Initializer)
{

}

void UChooserTable::PostLoad()
{
	Super::PostLoad();

	for (FInstancedStruct& ColumnData : ColumnsStructs)
	{
		if (ColumnData.IsValid())
		{
			FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
			Column.PostLoad();
		}
	}

#if WITH_EDITORONLY_DATA
	CachedPreviousOutputObjectType = OutputObjectType;
	CachedPreviousResultType = ResultType;

	if (Version < 1)
	{
		if (ParentTable)
		{
			RootChooser = ParentTable;
			ParentTable = nullptr;
		
			SetFlags(RF_Transactional); // fix nested chooser objects not created with Transactional flag
		
			// fix for broken outer object on nested chooser tables
			if (GetOuter() == GetPackage())
			{
				Rename(nullptr, RootChooser);
			}
		}

		if (RootChooser == nullptr && NestedChoosers.IsEmpty() && NestedObjects.IsEmpty())
		{
			// data upgrade for root tables: add elements to NestedTables list
			TArray<UObject*> ChildObjects;
			GetObjectsWithOuter(GetPackage(), ChildObjects, EGetObjectsFlags::IncludeNestedObjects);
			for (UObject* ChildObject : ChildObjects)
			{
				if (UChooserTable* Chooser = Cast<UChooserTable>(ChildObject))
				{
					if (Chooser != this)
					{
						NestedChoosers.Add(Chooser);
					}
				}
			}
		}
		Version = 1;
	}
	
	if (Version < 2 && NestedObjects.IsEmpty())
	{
		for(UChooserTable* NestedChooser : NestedChoosers)
		{
			NestedObjects.Add(NestedChooser);
		}
		Version = 2;
	}

	if (Version < 3)
	{
		NestedChoosers.Reset();
		Version = 3;
	}
#endif

	Compile();
}

void UChooserTable::BeginDestroy()
{
	ColumnsStructs.Empty();
#if WITH_EDITORONLY_DATA
	ResultsStructs.Empty();

	NestedChoosers.Empty();
	NestedObjects.Empty();
#endif
	CookedResults.Empty();
	Super::BeginDestroy();
}

#if WITH_EDITOR

void UChooserTable::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UChooserTable::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Output property names we use
	TStringBuilder<256> PropertyNamesBuilder;
	PropertyNamesBuilder.Append(PropertyTagDelimiter);

	for(const FInstancedStruct& Column : ColumnsStructs)
	{
		if (const FChooserColumnBase* ColumnBase = Column.GetPtr<FChooserColumnBase>())
		{
			if (FChooserParameterBase* Parameter = const_cast<FChooserColumnBase*>(ColumnBase)->GetInputValue())
			{
				Parameter->AddSearchNames(PropertyNamesBuilder);
			}
		}
	}

	Context.AddTag(FAssetRegistryTag(PropertyNamesTag, PropertyNamesBuilder.ToString(), FAssetRegistryTag::TT_Hidden));
}

void UChooserTable::AddCompileDependency(const UStruct* InStructType)
{
	UStruct* StructType = const_cast<UStruct*>(InStructType);
	if (!CompileDependencies.Contains(StructType))
	{
		if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(StructType))
		{
			UserDefinedStruct->ChangedEvent.AddUObject(this, &UChooserTable::OnDependentStructChanged);
			CompileDependencies.Add(StructType);
		}
		else if (UClass* Class = Cast<UClass>(StructType))
		{
			if(UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy))
			{
				Blueprint->OnCompiled().AddUObject(this, &UChooserTable::OnDependencyCompiled);
				CompileDependencies.Add(StructType);
			}
		}
	}
}

#endif

void UChooserTable::Compile(bool bForce)
{
	IHasContextClass* ContextOwner = GetRootChooser();
	
	for (FInstancedStruct& ColumnData : ColumnsStructs)
	{
		if (ColumnData.IsValid())
		{
			FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
			Column.Compile(ContextOwner, bForce);
		}
	}

#if WITH_EDITORONLY_DATA
	for(FInstancedStruct& ResultData : ResultsStructs)
	{
		if (ResultData.IsValid())
		{
			FObjectChooserBase& Result = ResultData.GetMutable<FObjectChooserBase>();
			Result.Compile(ContextOwner, bForce);
		}
	}
#endif

	if (IsCookedData())
	{
		for(FInstancedStruct& ResultData : CookedResults)
		{
			if (ResultData.IsValid())
			{
				FObjectChooserBase& Result = ResultData.GetMutable<FObjectChooserBase>();
				Result.Compile(ContextOwner, bForce);
			}
		}
	}
}

void UChooserTable::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving())
	{
		PopulateCookedData(Ar.IsCooking());
	}
#endif
	
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		// convert old data if it exists

		if (ContextObjectType_DEPRECATED)
		{
			ContextData.SetNum(1);
			ContextData[0].InitializeAs<FContextObjectTypeClass>();
			FContextObjectTypeClass& Context = ContextData[0].GetMutable<FContextObjectTypeClass>();
			Context.Class = ContextObjectType_DEPRECATED;
			Context.Direction = EContextObjectDirection::ReadWrite;
			ContextObjectType_DEPRECATED = nullptr;
		}

		if (Results_DEPRECATED.Num() > 0 || Columns_DEPRECATED.Num() > 0)
		{
			ResultsStructs.Reserve(Results_DEPRECATED.Num());
			ColumnsStructs.Reserve(Columns_DEPRECATED.Num());

			for (TScriptInterface<IObjectChooser>& Result : Results_DEPRECATED)
			{
				ResultsStructs.SetNum(ResultsStructs.Num() + 1);
				IObjectChooser* ResultInterface = Result.GetInterface();
				if (ResultInterface)
				{
					ResultInterface->ConvertToInstancedStruct(ResultsStructs.Last());
				}
			}

			for (TScriptInterface<IChooserColumn>& Column : Columns_DEPRECATED)
			{
				ColumnsStructs.SetNum(ColumnsStructs.Num() + 1);
				IChooserColumn* ColumnInterface = Column.GetInterface();
				if (ColumnInterface)
				{
					ColumnInterface->ConvertToInstancedStruct(ColumnsStructs.Last());
				}
			}

			Results_DEPRECATED.SetNum(0);
			Columns_DEPRECATED.SetNum(0);
		}

		bIsCookedPackage = Ar.IsLoadingFromCookedPackage();
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UChooserTable::RemoveDisabledData()
{
	// remove disabled or invalid columns
	ColumnsStructs.RemoveAll([](const FInstancedStruct& ColumnStruct)
	{
		return !ColumnStruct.IsValid() || ColumnStruct.Get<FChooserColumnBase>().bDisabled;
	});
	
	// remove disabled rows and corresponding row data from columns
	TArray<int> RowsToDelete;
	const int NumResults = ResultsStructs.Num();
	for (int ResultIndex = NumResults - 1; ResultIndex>=0; ResultIndex--)
	{
		if (IsRowDisabled(ResultIndex))
		{
			RowsToDelete.Add(ResultIndex);
		}
	}
	
	DisabledRows.SetNum(0);

	for (uint32 Index : RowsToDelete)
	{
		ResultsStructs.RemoveAt(Index);
	}
	for(FInstancedStruct& Column : ColumnsStructs)
	{
		Column.GetMutable<FChooserColumnBase>().DeleteRows(RowsToDelete);
	}
}

void UChooserTable::PopulateCookedData(bool bCooking)
{
	if (bCooking)
	{
		RemoveDisabledData();

		// copy stripped results struct into CookedResults array
		CookedResults = ResultsStructs;
	}
	else
	{
		// Build CookedResults array from stripped results struct.
		// When we aren't cooking, this is used to make sure the actually cooked rows show up as non-editor-only in tools and packaging code.
		CookedResults.Empty();
		for (int32 ResultIndex = 0; ResultIndex < ResultsStructs.Num(); ++ResultIndex)
		{
			if (!IsRowDisabled(ResultIndex))
			{
				CookedResults.Add(ResultsStructs[ResultIndex]);
			}
		}
	}
}


// Replace any references to a nested chooser, from either the Result column, or an OutputObject column
static void ReplaceReferencesInTable(UChooserTable* ChooserToReplace, UChooserTable* ReplaceWith, UChooserTable* Table)
{
	Table->Modify();
	
	for(FInstancedStruct& ResultData : Table->ResultsStructs)
	{
		if (FNestedChooser* NestedChooserResult = ResultData.GetMutablePtr<FNestedChooser>())
		{
			if (NestedChooserResult->Chooser == ChooserToReplace)
			{
				NestedChooserResult->Chooser = ReplaceWith;
			}
		}

		if (FNestedChooser* NestedChooserFallback = Table->FallbackResult.GetMutablePtr<FNestedChooser>())
		{
			if (NestedChooserFallback->Chooser == ChooserToReplace)
			{
				NestedChooserFallback->Chooser = ReplaceWith;
			}
		}
	}
	
	for (FInstancedStruct& ColumnData : Table->ColumnsStructs)
	{
		if (FOutputObjectColumn* OutputObjectColumn = ColumnData.GetMutablePtr<FOutputObjectColumn>())
		{
			for(FChooserOutputObjectRowData& RowData : OutputObjectColumn->RowValues)
			{
				if (FNestedChooser* NestedChooserResult = RowData.Value.GetMutablePtr<FNestedChooser>())
				{
					if (NestedChooserResult->Chooser == ChooserToReplace)
					{
						NestedChooserResult->Chooser = ReplaceWith;
					}
				}	
			}
			
			if (FNestedChooser* NestedChooserFallback = OutputObjectColumn->FallbackValue.Value.GetMutablePtr<FNestedChooser>())
			{
				if (NestedChooserFallback->Chooser == ChooserToReplace)
				{
					NestedChooserFallback->Chooser = ReplaceWith;
				}
			}
			
			if (FNestedChooser* NestedChooserDefault = OutputObjectColumn->DefaultRowValue.Value.GetMutablePtr<FNestedChooser>())
			{
				if (NestedChooserDefault->Chooser == ChooserToReplace)
				{
					NestedChooserDefault->Chooser = ReplaceWith;
				}
			}	
		}
	}

	for (FInstancedStruct& ColumnData : Table->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.ReplaceReferences(ChooserToReplace, ReplaceWith);
	}

	Table->PostEditChange();
}

static void ReplaceReferences(UChooserTable* ChooserToReplace, UChooserTable* ReplaceWith, UChooserTable* RootTable)
{
	ReplaceReferencesInTable(ChooserToReplace, ReplaceWith, RootTable);

	for(UObject* NestedObject : RootTable->NestedObjects)
	{
		if(UChooserTable* NestedChooser = Cast<UChooserTable>(NestedObject))
		{
			ReplaceReferencesInTable(ChooserToReplace, ReplaceWith, NestedChooser);

			// reparent any child tables to the root
			if (NestedChooser->GetOuter() == ChooserToReplace)
			{
				NestedChooser->Rename(nullptr, RootTable);
			}
		}
	}
}

void UChooserTable::DeleteNestedChooser(UChooserTable* ChooserToDelete)
{
	static FName DeletedChooserName = "DeletedNestedChooser";
	DeletedChooserName.SetNumber(DeletedChooserName.GetNumber() + 1);
	ChooserToDelete->Rename(*DeletedChooserName.ToString());
	Modify(true);
	RemoveNestedChooser(ChooserToDelete);
	ReplaceReferences(ChooserToDelete, nullptr, this);
}

static void RemoveReferencesFromColumns(UObject* ObjectToDelete, UChooserTable* ChooserTable)
{
	for (FInstancedStruct& ColumnData : ChooserTable->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.ReplaceReferences(ObjectToDelete, nullptr);
	}

	for (UObject* NestedObject : ChooserTable->NestedObjects)
	{
		if (UChooserTable* NestedChooser = Cast<UChooserTable>(NestedObject))
		{
			for (FInstancedStruct& ColumnData : NestedChooser->ColumnsStructs)
			{
				FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
				Column.ReplaceReferences(ObjectToDelete, nullptr);
			}
		}
	}
}

void UChooserTable::DeleteNestedObject(UObject* ObjectToDelete)
{
	static FName DeletedObjectName = "DeletedNestedObject";
	DeletedObjectName.SetNumber(DeletedObjectName.GetNumber() + 1);
	ObjectToDelete->Rename(*DeletedObjectName.ToString());
	Modify(true);
	RemoveNestedObject(ObjectToDelete);
	RemoveReferencesFromColumns(ObjectToDelete, this);
	PostEditChange();
}

#endif

#if WITH_EDITOR

EDataValidationResult UChooserTable::IsDataValid(FDataValidationContext& Context) const
{
	// todo: implement a const Compile which knows if it passed or failed
	const_cast<UChooserTable*>(this)->Compile(true);
	
	return Super::IsDataValid(Context);
}

void UChooserTable::PostEditUndo()
{
	UObject::PostEditUndo();

	if (CachedPreviousOutputObjectType != OutputObjectType || CachedPreviousResultType != ResultType)
	{
		OnOutputObjectTypeChanged.Broadcast(OutputObjectType);
		CachedPreviousOutputObjectType = OutputObjectType;
		CachedPreviousResultType = ResultType;
	}
	OnContextClassChanged.Broadcast();
}

void UChooserTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		static FName OutputObjectTypeName = "OutputObjectType";
		static FName ResultTypeName = "ResultType";
		if (PropertyChangedEvent.Property->GetName() == OutputObjectTypeName)
		{
			if (CachedPreviousOutputObjectType != OutputObjectType)
			{
				OnOutputObjectTypeChanged.Broadcast(OutputObjectType);
				CachedPreviousOutputObjectType = OutputObjectType;
			}
		}
		else if (PropertyChangedEvent.Property->GetName() == ResultTypeName)
		{
			if (CachedPreviousResultType != ResultType)
			{
				OnOutputObjectTypeChanged.Broadcast(OutputObjectType);
				CachedPreviousResultType = ResultType;
			}
		}
		else
		{
			OnContextClassChanged.Broadcast();
		}
	}
	else
	{
		OnOutputObjectTypeChanged.Broadcast(OutputObjectType);
		OnContextClassChanged.Broadcast();
	}
}

void UChooserTable::AddRecentContextObject(const FString& ObjectName) const
{
	FScopeLock Lock(&DebugLock);
	RecentContextObjects.AddUnique(ObjectName);
}

void UChooserTable::IterateRecentContextObjects(TFunction<void(const FString&)> Callback) const
{
	FScopeLock Lock(&DebugLock);
	RecentContextObjects.StableSort();
	for (const FString& ObjectName : RecentContextObjects)
	{
		Callback(ObjectName);
	}
}

void UChooserTable::UpdateDebugging(FChooserEvaluationContext& Context) const
{
	const UChooserTable* RootTable = GetRootChooser();

	for (const FStructView& Param : Context.Params)
	{
		if (const FChooserEvaluationInputObject* ObjectParam = Param.GetPtr<const FChooserEvaluationInputObject>())
		{
			if (UObject* ContextObject = ObjectParam->Object)
			{
				FString DebugName = ContextObject->GetName();
				if (UObject* Outer = ContextObject->GetTypedOuter(AActor::StaticClass()))
				{
					DebugName += " in " + Outer->GetName();					
				}

				if (UWorld* World = ContextObject->GetWorld())
				{
					if (World->IsPreviewWorld())
					{
						DebugName += " (Preview)";
					}
					else if (World->GetNetMode() == ENetMode::NM_DedicatedServer)
					{
						DebugName += " (Server)";
					}
					else if (World->GetNetMode() == ENetMode::NM_Client)
					{
						DebugName += FString(" (Client ") + FString::FromInt(World->GetOutermost()->GetPIEInstanceID()) + ")";
					}
				}

				AddRecentContextObject(DebugName);

				if (DebugName == RootTable->GetDebugTargetName())
				{
					bDebugTestValuesValid = true;
					Context.DebuggingInfo.bCurrentDebugTarget = true;
					return;
				}
			}
		}
	}
	Context.DebuggingInfo.bCurrentDebugTarget = false;
}

#endif

bool UChooserTable::ResultAssetFilter(const FAssetData& AssetData)
{
	return !AssetData.IsInstanceOf(OutputObjectType);
}

FObjectChooserBase::EIteratorStatus UChooserTable::EvaluateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback)
{
	return EvaluateChooser(Context, Chooser, FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback::CreateLambda([Callback](const TSoftObjectPtr<UObject>& ObjectPtr)
	{
		return Callback.Execute(ObjectPtr.Get());
	}));
}

FObjectChooserBase::EIteratorStatus UChooserTable::EvaluateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser,
	FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EvaluateChooser);

	if (Chooser == nullptr)
	{
		return FObjectChooserBase::EIteratorStatus::Failed;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Chooser->GetName());

	VALIDATE_CHOOSER_CONTEXT(Chooser, Chooser->ContextData, Context);

#if WITH_EDITOR
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EvaluateChooser_Debugging);
		Chooser->UpdateDebugging(Context);
	}
#endif
	
#if CHOOSER_DEBUGGING_ENABLED
	Context.DebuggingInfo.CurrentChooser = Chooser;
#endif

	const TArray<FInstancedStruct>* ResultsArray = &Chooser->CookedResults;

#if WITH_EDITORONLY_DATA
	if (!Chooser->IsCookedData())
	{
		ResultsArray = &Chooser->ResultsStructs;
	}
#endif
	

	uint32 Count = ResultsArray->Num();
	uint32 BufferSize = Count * sizeof(FChooserIndexArray::FIndexData);

	FChooserIndexArray Indices1(static_cast<FChooserIndexArray::FIndexData*>(FMemory_Alloca(BufferSize)), Count);
	FChooserIndexArray Indices2(static_cast<FChooserIndexArray::FIndexData*>(FMemory_Alloca(BufferSize)), Count);

	for(uint32 i=0;i<Count;i++)
	{
		if (!Chooser->IsRowDisabled(i))
		{
			Indices1.Push({i, 0});
		}
	}

	// calculating the necessary scratch area size and allocating it on the stack
	static constexpr int32 ScratchAreaAlignment = 16;
	int32 TotalScratchAreaSize = 0;
	for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
	{
		const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
		TotalScratchAreaSize += Align(Column.GetScratchAreaSize(), ScratchAreaAlignment);
	}
	TArrayView<uint8> ScratchArea((uint8*)FMemory_Alloca_Aligned(TotalScratchAreaSize * sizeof(uint8), ScratchAreaAlignment), TotalScratchAreaSize);
	auto DeinitializeScratchAreas = [Chooser, ScratchArea, TotalScratchAreaSize]()
		{
			if (TotalScratchAreaSize > 0)
			{
				// deinitializing all the ScratchArea(s)
				int32 ScratchAreaStart = 0;
				for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
				{
					const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
					const int32 ColumnScratchAreaSize = Column.GetScratchAreaSize();

					// @todo: maybe initialize ScratchArea only for column that has filters and are not bDisabled
					TArrayView<uint8> ColumnScratchArea = ScratchArea.Slice(ScratchAreaStart, ColumnScratchAreaSize);
					Column.DeinitializeScratchArea(ColumnScratchArea);
					ScratchAreaStart += Align(ColumnScratchAreaSize, ScratchAreaAlignment);
				}
			}
		};

	FChooserIndexArray* IndicesOut = &Indices1;
	FChooserIndexArray* IndicesIn = &Indices2;

	int32 ScratchAreaStart = 0;
	for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
	{
		const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
		const int32 ColumnScratchAreaSize = Column.GetScratchAreaSize();

		// @todo: maybe initialize ScratchArea only for column that has filters and are not bDisabled
		TArrayView<uint8> ColumnScratchArea = ScratchArea.Slice(ScratchAreaStart, ColumnScratchAreaSize);
		Column.InitializeScratchArea(ColumnScratchArea);
		ScratchAreaStart += Align(ColumnScratchAreaSize, ScratchAreaAlignment);

#if WITH_EDITORONLY_DATA
		if (Column.bDisabled)
		{
			continue;
		}
#endif
		
		if (Column.HasFilters())
		{
			Swap(IndicesIn, IndicesOut);
			IndicesOut->SetNum(0);
			Column.Filter(Context, *IndicesIn, *IndicesOut, ColumnScratchArea);
			
			if (IndicesIn->HasCosts() || Column.HasCosts())
			{
				IndicesOut->SetHasCosts();
			}
		}
	}
	check(ScratchAreaStart == TotalScratchAreaSize);

	//	No need to score with only one valid option
	if (IndicesOut->Num() > 1 && IndicesOut->HasCosts())
	{
		Algo::Sort(*IndicesOut);
	}

	bool bAnyRowSucceeded = false;
	
	// of the rows that passed all column filters, iterate through them calling the callback until it returns Stop
	for (FChooserIndexArray::FIndexData& SelectedIndexData : *IndicesOut)
	{
		if (ResultsArray->IsValidIndex(SelectedIndexData.Index))
		{
			// Set output columns before calling child
			/// this changes previous behavior, but enables both: child table output columns to override parent table output columns, and capturing multiple output struct values in mult-output mode
			{
				ScratchAreaStart = 0;
				for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
				{
					const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
					const int32 ColumnScratchAreaSize = Column.GetScratchAreaSize();

					TArrayView<uint8> ColumnScratchArea = ScratchArea.Slice(ScratchAreaStart, ColumnScratchAreaSize);
					ScratchAreaStart += Align(ColumnScratchAreaSize, ScratchAreaAlignment);

					#if WITH_EDITORONLY_DATA
					if (Column.bDisabled)
					{
						continue;
					}
					#endif
					
					Column.SetOutputs(Context, SelectedIndexData.Index, ColumnScratchArea);
				}
				check(ScratchAreaStart == TotalScratchAreaSize);
			}

			if (const FObjectChooserBase* SelectedResult = (*ResultsArray)[SelectedIndexData.Index].GetPtr<FObjectChooserBase>())
			{
				FObjectChooserBase::EIteratorStatus Status = SelectedResult->ChooseMulti(Context, Callback);
				
				if (Status != FObjectChooserBase::EIteratorStatus::Failed)
				{
					bAnyRowSucceeded = true;
				}
	
				if (Status == FObjectChooserBase::EIteratorStatus::Stop)
				{
					DeinitializeScratchAreas();
					TRACE_CHOOSER_EVALUATION(Chooser, Context, SelectedIndexData.Index);
	#if WITH_EDITOR
					Chooser->SetDebugSelectedRow(SelectedIndexData.Index);
	#endif
					return FObjectChooserBase::EIteratorStatus::Stop;
				}
			}
		}
	}

	// if no rows passed output fallback result
	if (!bAnyRowSucceeded)
	{
		TRACE_CHOOSER_EVALUATION(Chooser, Context, ChooserColumn_SpecialIndex_Fallback);
#if WITH_EDITOR
		Chooser->SetDebugSelectedRow(ChooserColumn_SpecialIndex_Fallback);
#endif
		
		if (Chooser->FallbackResult.IsValid())
		{
			const FObjectChooserBase& SelectedResult = Chooser->FallbackResult.Get<FObjectChooserBase>();
			FObjectChooserBase::EIteratorStatus Status = SelectedResult.ChooseMulti(Context, Callback);
			if (Status != FObjectChooserBase::EIteratorStatus::Failed)
			{
				// trigger all output columns to set their default output value
				ScratchAreaStart = 0;
				for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
				{
					const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
					const int32 ColumnScratchAreaSize = Column.GetScratchAreaSize();

					TArrayView<uint8> ColumnScratchArea = ScratchArea.Slice(ScratchAreaStart, ColumnScratchAreaSize);
					ScratchAreaStart += Align(ColumnScratchAreaSize, ScratchAreaAlignment);
					Column.SetOutputs(Context, ChooserColumn_SpecialIndex_Fallback, ColumnScratchArea);
				}
				check(ScratchAreaStart == TotalScratchAreaSize);
			}
			if (Status == FObjectChooserBase::EIteratorStatus::Stop)
			{
				DeinitializeScratchAreas();

				return FObjectChooserBase::EIteratorStatus::Stop;
			}
		}
	}

#if WITH_EDITOR
	if (Context.DebuggingInfo.bCurrentDebugTarget)
	{
		TArray<int32> SelectedIndices;
		SelectedIndices.Reserve(FMath::Max(IndicesOut->Num(), 1u));
		if (IndicesOut->Num() == 0)
		{
			SelectedIndices.Add(ChooserColumn_SpecialIndex_Fallback);
		}
		else
		{
			for(const FChooserIndexArray::FIndexData& IndexEntry : *IndicesOut)
			{
				SelectedIndices.Add(IndexEntry.Index);		
			}
		}
		
		Chooser->SetDebugSelectedRows(SelectedIndices);
	}
#endif
			
	TRACE_CHOOSER_EVALUATION(Chooser, Context, *IndicesOut);

	DeinitializeScratchAreas();

	return bAnyRowSucceeded ? FObjectChooserBase::EIteratorStatus::Continue : FObjectChooserBase::EIteratorStatus::Failed;
}

FObjectChooserBase::EIteratorStatus UChooserTable::IterateChooser(const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback)
{
	FChooserEvaluationContext Context;
	return IterateChooser(Context, Chooser, Callback);
}

FObjectChooserBase::EIteratorStatus UChooserTable::IterateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IterateChooser);

	if (Chooser == nullptr)
	{
		return FObjectChooserBase::EIteratorStatus::Continue;
	}

	const TArray<FInstancedStruct>* ResultsArray = &Chooser->CookedResults;

#if WITH_EDITORONLY_DATA
	if (!Chooser->IsCookedData())
	{
		ResultsArray = &Chooser->ResultsStructs;
	}
#endif
	
	uint32 Count = ResultsArray->Num();
	uint32 BufferSize = Count * sizeof(FChooserIndexArray::FIndexData);

	FChooserIndexArray Indices(static_cast<FChooserIndexArray::FIndexData*>(FMemory_Alloca(BufferSize)), Count);

	for(uint32 i=0;i<Count;i++)
	{
		if (!Chooser->IsRowDisabled(i))
		{
			Indices.Push({i, 0});
		}
	}

	for (FChooserIndexArray::FIndexData& SelectedIndexData : Indices)
	{
		if (ResultsArray->IsValidIndex(SelectedIndexData.Index))
		{
			const FObjectChooserBase& SelectedResult = (*ResultsArray)[SelectedIndexData.Index].Get<FObjectChooserBase>();

			for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();

				#if WITH_EDITORONLY_DATA
				if (Column.bDisabled)
				{
					continue;
				}
				#endif
				
				Column.SetOutputs(Context, SelectedIndexData.Index);
			}

			FObjectChooserBase::EIteratorStatus Status = SelectedResult.IterateObjects(Context, Callback);
			if (Status == FObjectChooserBase::EIteratorStatus::Stop)
			{
				return FObjectChooserBase::EIteratorStatus::Stop;
			}
		}
	}

	if (Chooser->FallbackResult.IsValid())
	{
		const FObjectChooserBase& SelectedResult = Chooser->FallbackResult.Get<FObjectChooserBase>();
		for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
			Column.SetOutputs(Context, ChooserColumn_SpecialIndex_Fallback);
		}

		return SelectedResult.IterateObjects(Context, Callback);
	}

	return FObjectChooserBase::EIteratorStatus::Continue;
}

void FEvaluateChooser::ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const
{
    UChooserTable::EvaluateChooser(Context, Chooser, FObjectChooserSoftObjectIteratorCallback::CreateLambda([&Result](const TSoftObjectPtr<UObject>& InResult)
    {
    	Result = InResult;
    	return FObjectChooserBase::EIteratorStatus::Stop;
    }));
}

UObject* FEvaluateChooser::ChooseObject(FChooserEvaluationContext& Context) const
{
	UObject* Result = nullptr;
	UChooserTable::EvaluateChooser(Context, Chooser, FObjectChooserIteratorCallback::CreateLambda([&Result](UObject* InResult)
	{
		Result = InResult;
		return FObjectChooserBase::EIteratorStatus::Stop;
	}));

	return Result;
}

FObjectChooserBase::EIteratorStatus FEvaluateChooser::ChooseMulti(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return UChooserTable::EvaluateChooser(Context, Chooser, Callback);
}

FObjectChooserBase::EIteratorStatus FEvaluateChooser::ChooseMulti(FChooserEvaluationContext& Context,
	FObjectChooserSoftObjectIteratorCallback Callback) const
{
	return UChooserTable::EvaluateChooser(Context, Chooser, Callback);
}

FObjectChooserBase::EIteratorStatus FEvaluateChooser::IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return UChooserTable::IterateChooser(Context, Chooser, Callback);
}

void FEvaluateChooser::GetDebugName(FString& OutDebugName) const
{
	if (Chooser)
	{
		OutDebugName = Chooser.GetName();
	}
}


FNestedChooser::FNestedChooser()
{
}

void FNestedChooser::ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const
{
    UChooserTable::EvaluateChooser(Context, Chooser, FObjectChooserSoftObjectIteratorCallback::CreateLambda([&Result](const TSoftObjectPtr<UObject>& InResult)
    {
    	Result = InResult;
    	return FObjectChooserBase::EIteratorStatus::Stop;
    }));
}

UObject* FNestedChooser::ChooseObject(FChooserEvaluationContext& Context) const
{
	UObject* Result = nullptr;
	UChooserTable::EvaluateChooser(Context, Chooser, FObjectChooserIteratorCallback::CreateLambda([&Result](UObject* InResult)
	{
		Result = InResult;
		return FObjectChooserBase::EIteratorStatus::Stop;
	}));

	return Result;
}

FObjectChooserBase::EIteratorStatus FNestedChooser::ChooseMulti(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return UChooserTable::EvaluateChooser(Context, Chooser, Callback);
}

FObjectChooserBase::EIteratorStatus FNestedChooser::ChooseMulti(FChooserEvaluationContext& Context, FObjectChooserSoftObjectIteratorCallback Callback) const
{
	return UChooserTable::EvaluateChooser(Context, Chooser, Callback);
}

FObjectChooserBase::EIteratorStatus FNestedChooser::IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return UChooserTable::IterateChooser(Context, Chooser, Callback);
}

void FNestedChooser::GetDebugName(FString& OutDebugName) const
{
	OutDebugName  = GetNameSafe(Chooser);
}

#if WITH_EDITORONLY_DATA
namespace {
	void TestCook()
	{
		for (TObjectIterator<UChooserTable> It; It; ++It)
		{
			It->PopulateCookedData(true);
		}
	}
}

static FAutoConsoleCommand CCmdTestCookChoosers(
	TEXT("Chooser.TestCook"),
	TEXT(""),
	FConsoleCommandDelegate::CreateStatic(TestCook));


void UChooserTable::AddNestedChooser(UChooserTable* Chooser)
{
	AddNestedObject(Chooser);
}

void UChooserTable::AddNestedObject(UObject* Object)
{
	check(Object);
	ensure(!RootChooser);
	ensure(!NestedObjects.Contains(Object));

	FName NewObjectName = Object->GetFName();
	FString NewChooserPlainName = NewObjectName.GetPlainNameString();
	bool bNameConflict = false;
	int MaxNumber = NewObjectName.GetNumber();
	for(UObject* NestedObjectIterator : NestedObjects)
	{
		FName NestedObjectName = NestedObjectIterator->GetFName();
		if (NestedObjectName.GetPlainNameString() == NewChooserPlainName)
		{
			MaxNumber = FMath::Max(MaxNumber, NestedObjectName.GetNumber());
			if(NestedObjectName.GetNumber() == NewObjectName.GetNumber())
			{
				bNameConflict = true;
			}
		}
	}
	if (bNameConflict)
	{
		NewObjectName.SetNumber(MaxNumber + 1);
		Object->Rename(*NewObjectName.ToString());
	}
	
	NestedObjects.Add(Object);
	NestedObjectsChanged.Broadcast();
}

void UChooserTable::RemoveNestedChooser(UChooserTable* Chooser)
{
	// deleting the Chooser by removing all the rows and resetting ResultsStructs / DisabledRows / FallbackResult
	// so columns are aware of dependencies being changed
	TArray<int32> RowsToDelete;
	RowsToDelete.Reserve(Chooser->ResultsStructs.Num() + 1);
	for (int32 RowIndex = Chooser->ResultsStructs.Num() - 1; RowIndex >= 0; --RowIndex)
	{
		RowsToDelete.Add(RowIndex);
	}
	RowsToDelete.Add(ChooserColumn_SpecialIndex_Fallback);
	
	// calling DeleteRows on columns before removing them from FallbackResult / DisabledRows / ResultsStructs,
	// so they are aware of what's gonna be deleted, and access those properties to remove references, etc.
	for(FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.DeleteRows(RowsToDelete);
	}

	Chooser->ResultsStructs.Reset();
	Chooser->DisabledRows.Reset();
	Chooser->FallbackResult.Reset();

	RemoveNestedObject(Chooser);
}

void UChooserTable::RemoveNestedObject(UObject* Object)
{
	ensure(NestedObjects.Contains(Object));
	NestedObjects.Remove(Object);
	NestedObjectsChanged.Broadcast();
}

#endif