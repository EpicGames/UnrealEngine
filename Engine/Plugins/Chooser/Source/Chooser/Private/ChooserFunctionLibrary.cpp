// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChooserFunctionLibrary.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Chooser.h"
#include "ChooserPropertyAccess.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserFunctionLibrary)


namespace
{
	void AnimInstanceSync(UAnimInstance* AnimInstance)
	{
		class UFinishFlyingAnimInstanceTasks : public UAnimInstance
		{
		public:
			static void Execute(UAnimInstance* AnimInstance)
			{
				check(AnimInstance);
				static_cast<UFinishFlyingAnimInstanceTasks*>(AnimInstance)->GetProxyOnAnyThread<FAnimInstanceProxy>();
			}
		};
		UFinishFlyingAnimInstanceTasks::Execute(AnimInstance);
	}
}


UChooserFunctionLibrary::UChooserFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

static FObjectChooserBase::EIteratorStatus StaticEvaluateChooser(const UObject* ContextObject, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback)
{
	// Fallback single context object version
	FChooserEvaluationContext Context(const_cast<UObject*>(ContextObject));
	return UChooserTable::EvaluateChooser(Context, Chooser, Callback);
}

UObject* UChooserFunctionLibrary::EvaluateChooser(const UObject* ContextObject, const UChooserTable* Chooser, TSubclassOf<UObject> ObjectClass)
{
	UObject* Result = nullptr;
	StaticEvaluateChooser(ContextObject, Chooser, FObjectChooserBase::FObjectChooserIteratorCallback::CreateLambda([&Result, ObjectClass](UObject* InResult)
	{
		if (InResult && (ObjectClass == nullptr || InResult->IsA(ObjectClass)))
		{
			Result = InResult;
			return FObjectChooserBase::EIteratorStatus::Stop;
		}
		return FObjectChooserBase::EIteratorStatus::Failed;
	}));

	return Result;
}

void UChooserFunctionLibrary::AddChooserObjectInput(FChooserEvaluationContext& Context, UObject* Object)
{
	Context.AddObjectParam(Object);
}

UObject* UChooserFunctionLibrary::GetChooserObjectInput(FChooserEvaluationContext& Context, int32 Index, TSubclassOf<UObject> ObjectClass)
{
	if (Index < Context.Params.Num())
	{
		if (FChooserEvaluationInputObject* ObjectParam = Context.Params[Index].GetPtr<FChooserEvaluationInputObject>())
		{
			return ObjectParam->Object;
		}
	}
	return nullptr;
}

UObject* UChooserFunctionLibrary::GetChooserObject(FChooserEvaluationContext& Context, TSubclassOf<UObject> ObjectClass)
{
	for (FStructView& ContextParam : Context.Params)
	{
		if (FChooserEvaluationInputObject* ObjectParam = ContextParam.GetPtr<FChooserEvaluationInputObject>())
		{
			if (UObject* ObjectParamObject = ObjectParam->Object)
			{
				if (ObjectParamObject->IsA(ObjectClass))
				{
					return ObjectParamObject;
				}

				if (const UClass* ObjectParamObjectClass = ObjectParamObject->GetClass())
				{
					if (ObjectParamObjectClass->ImplementsInterface(ObjectClass))
					{
						return ObjectParamObject;
					}
				}
			}
		}
	}
	return nullptr;
}

void UChooserFunctionLibrary::AddChooserStructInput(FChooserEvaluationContext& Inputs, int32 Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UChooserFunctionLibrary::execAddChooserStructInput)
{
	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	
	P_GET_STRUCT_REF(FChooserEvaluationContext, ChooserContext);
	
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	uint8* ValuePtr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	
	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("ChooserTable", "InstancedStruct_MakeInvalidValueWarning", "Failed to resolve the Value for Add Chooser Struct Input")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		ChooserContext.Params.Add(FStructView(ValueProp->Struct, ValuePtr));
		P_NATIVE_END;
	}
}

void UChooserFunctionLibrary::AddChooserStructInputOutput(FChooserEvaluationContext& Inputs, int32 Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UChooserFunctionLibrary::execAddChooserStructInputOutput)
{
	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	
	P_GET_STRUCT_REF(FChooserEvaluationContext, ChooserContext);
	
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	uint8* ValuePtr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	
	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("ChooserTable", "InstancedStruct_MakeInvalidValueWarning", "Failed to resolve the Value for Add Chooser Struct Input")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		ChooserContext.OutputArrays.Add({static_cast<uint32>(ChooserContext.Params.Num())});
		ChooserContext.Params.Add(FStructView(ValueProp->Struct, ValuePtr));
		P_NATIVE_END;
	}
}

void UChooserFunctionLibrary::GetChooserStructOutput(FChooserEvaluationContext& Context, int index, int32& Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UChooserFunctionLibrary::execGetChooserStructOutput)
{
	P_GET_STRUCT_REF(FChooserEvaluationContext, Outputs);
	P_GET_PROPERTY(FIntProperty, Index);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ValueProp = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("ChooserTable","GetChooserStructOutput_InvalidValueWarning", "Failed to resolve the Value for Get Chooser Struct Output")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		if (Outputs.Params.Num() > Index && Outputs.Params[Index].IsValid() && Outputs.Params[Index].GetScriptStruct()->IsChildOf(ValueProp->Struct))
		{
			ValueProp->Struct->CopyScriptStruct(ValuePtr, Outputs.Params[Index].GetMemory());
		}
		P_NATIVE_END;
	}
}

void UChooserFunctionLibrary::GetChooserStructOutputArray(FChooserEvaluationContext& Context, int index, TArray<int32>& Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UChooserFunctionLibrary::execGetChooserStructOutputArray)
{
	P_GET_STRUCT_REF(FChooserEvaluationContext, Outputs);
	P_GET_PROPERTY(FIntProperty, Index);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FArrayProperty>(nullptr);
	
	const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Stack.MostRecentProperty);
	void* ArrayPtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ArrayProp || !ArrayPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("ChooserTable","GetChooserStructOutput_InvalidValueWarning", "Failed to resolve the Value for Get Chooser Struct Output")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		if (const FStructProperty *StructProperty = CastField<FStructProperty>(ArrayProp->Inner))
		{
			if (Outputs.OutputArrays.IsValidIndex(Index))
			{
				TArray<FInstancedStruct> SourceArray = Outputs.OutputArrays[Index].OutputValues;
				
				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayPtr);
				ArrayHelper.EmptyAndAddValues(SourceArray.Num());

				int ElementIndex = 0;
				for (FInstancedStruct& SourceEntry : SourceArray)
				{
					uint8* TargetElementPtr = ArrayHelper.GetRawPtr(ElementIndex);
					if (SourceEntry.GetScriptStruct() == StructProperty->Struct)
					{
						StructProperty->Struct->CopyScriptStruct(TargetElementPtr, SourceEntry.GetMemory());
					}
					ElementIndex ++;
				}
			}
		}
		P_NATIVE_END;
	}
}

UObject* UChooserFunctionLibrary::EvaluateObjectChooserBase(FChooserEvaluationContext& Context, const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass)
{
 	UObject* Result = nullptr;
	
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetFirstObjectParam()))
	{
		AnimInstanceSync(AnimInstance);
	}

	if (const FObjectChooserBase* ObjectChooserPtr = ObjectChooser.GetPtr<FObjectChooserBase>())
	{
		ObjectChooserPtr->ChooseMulti(Context, FObjectChooserBase::FObjectChooserIteratorCallback::CreateLambda([&Result, bResultIsClass, ObjectClass](UObject* InResult)
		{
			if (InResult)
			{
				bool bTypesMatch = true;

				if (ObjectClass && InResult)
				{
					if (bResultIsClass)
					{
						UClass* ResultClass =  Cast<UClass>(InResult);
						bTypesMatch = ResultClass && ResultClass->IsChildOf(ObjectClass);
					}
					else
					{
						bTypesMatch = InResult->IsA(ObjectClass);
					}
				}

				if (bTypesMatch)
				{
					Result = InResult;
					return FObjectChooserBase::EIteratorStatus::Stop;
				}
			}
			return FObjectChooserBase::EIteratorStatus::Failed;
		}));
	}
	
 	return Result;
 }

TSoftObjectPtr<UObject> UChooserFunctionLibrary::EvaluateObjectChooserBaseSoft(FChooserEvaluationContext& Context, const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass)
{
	TSoftObjectPtr<UObject> Result;
	
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetFirstObjectParam()))
	{
		AnimInstanceSync(AnimInstance);
	}

	if (const FObjectChooserBase* ObjectChooserPtr = ObjectChooser.GetPtr<FObjectChooserBase>())
	{
		ObjectChooserPtr->ChooseObject(Context, Result);
	}
	
 	return Result;
}



TArray<UObject*> UChooserFunctionLibrary::EvaluateObjectChooserBaseMulti(FChooserEvaluationContext& Context, const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass)
{
 	TArray<UObject*> Result;
	
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetFirstObjectParam()))
	{
		AnimInstanceSync(AnimInstance);
	}

	if (const FObjectChooserBase* ObjectChooserPtr = ObjectChooser.GetPtr<FObjectChooserBase>())
	{
		ObjectChooserPtr->ChooseMulti(Context, FObjectChooserBase::FObjectChooserIteratorCallback::CreateLambda([&Result, &Context, bResultIsClass, ObjectClass](UObject* InResult)
		{
			bool bTypesMatch = true;
            
			if (ObjectClass && InResult)
			{
				if (bResultIsClass)
				{
					UClass* ResultClass =  Cast<UClass>(InResult);
					bTypesMatch = ResultClass && ResultClass->IsChildOf(ObjectClass);
				}
				else
				{
					bTypesMatch = InResult->IsA(ObjectClass);
				}
			}
            
			if (bTypesMatch)
			{
				Result.Add(InResult);

				// todo: move this into a context function
				for(auto& OutputArray : Context.OutputArrays)
				{
					OutputArray.OutputValues.Add(FInstancedStruct(Context.Params[OutputArray.SourceParamIndex]));
				}
				return FObjectChooserBase::EIteratorStatus::Continue;
			}
			else
			{
				return FObjectChooserBase::EIteratorStatus::Failed;
			}
		}));
	}
	
 	return Result;
}

TArray<TSoftObjectPtr<UObject>> UChooserFunctionLibrary::EvaluateObjectChooserBaseMultiSoft(FChooserEvaluationContext& Context, const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass)
{
 	TArray<TSoftObjectPtr<UObject>> Result;
	
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetFirstObjectParam()))
	{
		AnimInstanceSync(AnimInstance);
	}

	if (const FObjectChooserBase* ObjectChooserPtr = ObjectChooser.GetPtr<FObjectChooserBase>())
	{
		ObjectChooserPtr->ChooseMulti(Context, FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback::CreateLambda([&Result, &Context](const TSoftObjectPtr<UObject>& InResult)
		{
			Result.Add(InResult);
				
			// todo: move this into a context function
			for(auto& OutputArray : Context.OutputArrays)
			{
				OutputArray.OutputValues.Add(FInstancedStruct(Context.Params[OutputArray.SourceParamIndex]));
			}	
				
			return FObjectChooserBase::EIteratorStatus::Continue;
		}));
	}
	
 	return Result;
}

FInstancedStruct UChooserFunctionLibrary::MakeEvaluateChooser(UChooserTable* Chooser)
{
	FInstancedStruct Struct;
	Struct.InitializeAs(FEvaluateChooser::StaticStruct());
	Struct.GetMutable<FEvaluateChooser>().Chooser = Chooser;
	return Struct;
}

FChooserEvaluationContext UChooserFunctionLibrary::MakeChooserEvaluationContext()
{
	FChooserEvaluationContext Context;
	return Context;
}

TArray<UObject*> UChooserFunctionLibrary::EvaluateChooserMulti(const UObject* ContextObject, const UChooserTable* Chooser, TSubclassOf<UObject> ObjectClass)
{
	TArray<UObject*> Result;
	StaticEvaluateChooser(ContextObject, Chooser, FObjectChooserBase::FObjectChooserIteratorCallback::CreateLambda([&Result, ObjectClass](UObject* InResult)
	{
		if (InResult && (ObjectClass == nullptr || InResult->IsA(ObjectClass)))
		{
			Result.Add(InResult);
			return FObjectChooserBase::EIteratorStatus::Continue;
		}
		else
		{
			return FObjectChooserBase::EIteratorStatus::Failed;
		}
	}));

	for (int Index = 0; Index < Result.Num(); ++Index)
	{
		if (Result[Index] && !Result[Index]->IsA(ObjectClass))
		{
			Result[Index] = nullptr;
		}
	}

	return Result;
}