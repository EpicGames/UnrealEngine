// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserModule.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"
#include "Features/IModularFeatures.h"

void FChooserModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &ChooserExtension);
}

void FChooserModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &ChooserExtension);
}

IMPLEMENT_MODULE(FChooserModule, Chooser);
