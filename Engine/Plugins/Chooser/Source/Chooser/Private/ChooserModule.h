// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "RewindDebuggerChooserRuntime.h"

class FChooserModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	FRewindDebuggerChooserRuntime ChooserExtension;
};
