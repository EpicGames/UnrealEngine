// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTypes.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

bool FChooserPlayerSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}

	return false;
}

bool FChooserPlayerSettings::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.IsSerializingDefaults())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			BlendOption = EAlphaBlendOption::Linear;
		}
	}
	return false;
}
