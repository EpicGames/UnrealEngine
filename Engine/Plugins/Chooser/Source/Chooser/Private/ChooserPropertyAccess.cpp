// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserPropertyAccess.h"

#include "IObjectChooser.h"
#include "StructUtils/UserDefinedStruct.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UnrealType.h"
#include "Logging/LogMacros.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "IPropertyAccessEditor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserPropertyAccess)

#define LOCTEXT_NAMESPACE "ChooserPropertyAccess"

TAutoConsoleVariable<bool> CVarEnableDetailedWarnings(
	TEXT("Choosers.EnableDetailedWarnings"),
	true,
	TEXT("Enable detailed context validation with warnings when choosers are evaluated on an incorrect context. \n0: Disable, 1: Enable (default)"),
	ECVF_Default);

TAutoConsoleVariable<bool> CVarUseCompiledPropertyChainsInEditor(
	TEXT("Choosers.UseCompiledPropertyChainsInEditor"),
	false,
	TEXT("Enable optimized property access on Choosers and Proxy Tables in Editor. \n0: Disable (default), 1: Enable"),
	ECVF_Default);
		
FName FChooserPropertyBinding::GetUniqueId() const
{
	FString Result;
	bool First = true;
	for (const auto& Element : PropertyBindingChain)
	{
		if (First)
		{
			First = false;
		}
		else
		{
			Result += TEXT(".");
		}

		Result += Element.ToString();
	}

	return FName(Result);
}


struct FCompiledBindingCacheId
{
	FName BindingPath;
	const UStruct* Type = nullptr;
	int32 ContextIndex = 0;
	
	bool operator == (const FCompiledBindingCacheId& Other) const
	{
		 return ContextIndex == Other.ContextIndex
				&&Type == Other.Type
				&& BindingPath == Other.BindingPath;
	}
};

FORCEINLINE uint32 GetTypeHash(const FCompiledBindingCacheId& Id)
{
	return HashCombineFast(GetTypeHash(Id.ContextIndex),HashCombineFast(GetTypeHash(Id.Type), GetTypeHash(Id.BindingPath)));
}

TMap<FCompiledBindingCacheId, TWeakPtr<UE::Chooser::FCompiledBinding>> CompiledBindingCache;
FCriticalSection CompiledBindingCacheLock;

void FChooserPropertyBinding::Compile(IHasContextClass* Owner, bool bForce)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompilePropertyChain);

#if WITH_EDITORONLY_DATA
	CompileMessage = FText();
#endif
	
	TConstArrayView<FInstancedStruct> ContextData = Owner->GetContextData();

	int CompiledBindingSerialNumber = 0;
	
	if ((PropertyBindingChain.IsEmpty() && !IsBoundToRoot) || !ContextData.IsValidIndex(ContextIndex))
	{
#if WITH_EDITORONLY_DATA
		CompileMessage = LOCTEXT("No Property Bound", "No Property Bound");
#endif
		UE_ASSET_LOG(LogChooser, Error, Owner->GetContextOwnerAsset(), TEXT("Missing property binding."));
		CompiledBinding = nullptr;
		return;
	}

	const UStruct* StructType = nullptr;
	if (const FContextObjectTypeClass* ClassContext = ContextData[ContextIndex].GetPtr<FContextObjectTypeClass>())
	{
		StructType = ClassContext->Class;
	}
	else if (const FContextObjectTypeStruct* StructContext = ContextData[ContextIndex].GetPtr<FContextObjectTypeStruct>())
	{
		StructType = StructContext->Struct;
	}

	if(StructType == nullptr)
	{
#if WITH_EDITORONLY_DATA
		CompileMessage = FText::Format(LOCTEXT("No valid struct", "No valid Context Object/Struct at index: {0}"), FText::FromString(FString::FromInt(ContextIndex)));
#endif
		UE_ASSET_LOG(LogChooser, Error, Owner->GetContextOwnerAsset(), TEXT("No valid Context Object/Struct at index: %d"), ContextIndex);
		CompiledBinding = nullptr;
		return;
	}
		
	FCompiledBindingCacheId Id;
	Id.Type = StructType;
	Id.BindingPath = GetUniqueId();
	Id.ContextIndex = ContextIndex;

	{
		FScopeLock Lock(&CompiledBindingCacheLock);
		if (TWeakPtr<UE::Chooser::FCompiledBinding>* Binding =  CompiledBindingCache.Find(Id))
		{
			if (Binding->IsValid())
			{
				bool bUseCachedBinding = true;
#if WITH_EDITOR
				if (bForce && CompiledBinding)
				{
					// we've triggered a recompile due to dependency changes.
					// if the one in the cache has a higher serial number, then it has already been recompiled by some other reference, so use the cached one.
					// otherwise, remove the currently cached one, recompile, and increment the serial number
					if (Binding->Pin()->SerialNumber <= CompiledBinding->SerialNumber)
					{
						bUseCachedBinding = false;
						CompiledBindingSerialNumber = CompiledBinding->SerialNumber + 1;
						CompiledBindingCache.Remove(Id);
					}
				}
#endif

				if (bUseCachedBinding)
				{
					CompiledBinding = Binding->Pin();

#if WITH_EDITOR
					for (const UStruct* Dependency : CompiledBinding->Dependencies)
					{
						Owner->AddCompileDependency(Dependency);
					}
#endif
					return;
				}
			}
		}
	}

	TSharedPtr<UE::Chooser::FCompiledBinding> NewCompiledBinding = MakeShared<UE::Chooser::FCompiledBinding>();
	UE::Chooser::FCompiledBinding& OutCompiledBinding = *NewCompiledBinding.Get();

	OutCompiledBinding.TargetType = StructType;
	OutCompiledBinding.CompiledChain.SetNum(0);
	OutCompiledBinding.ContextIndex = ContextIndex;
	
	const int PropertyChainLength = PropertyBindingChain.Num();
	int CurrentOffset = 0;
	for(int PropertyChainIndex = 0; PropertyChainIndex < PropertyChainLength - 1; PropertyChainIndex++)
	{
#if WITH_EDITOR
		Owner->AddCompileDependency(StructType);
		OutCompiledBinding.Dependencies.AddUnique(StructType);
#endif

		bool bFound = false;
		if (const FStructProperty* StructProperty = FindFProperty<FStructProperty>(StructType, PropertyBindingChain[PropertyChainIndex]))
		{
			bFound = true;
			// accumulate offsets in structs
			CurrentOffset += StructProperty->GetOffset_ForInternal();
			StructType = StructProperty->Struct;
		}
		else if (const FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(StructType, PropertyBindingChain[PropertyChainIndex]))
		{
			bFound = true;
			// when we hit an  object reference, create a chain element with the current offset
			CurrentOffset += ObjectProperty->GetOffset_ForInternal();
			OutCompiledBinding.CompiledChain.Add(UE::Chooser::FCompiledBindingElement(CurrentOffset));
			StructType = ObjectProperty->PropertyClass;
			// clear the offset, to start accumulating again relative to the new object base
			CurrentOffset = 0;
		}
		// check if it's a member function
		else if (const UClass* ClassType = Cast<const UClass>(StructType))
		{
			if (UFunction* Function = ClassType->FindFunctionByName(PropertyBindingChain[PropertyChainIndex]))
			{
				bFound = true;
				ensure(CurrentOffset == 0);
				OutCompiledBinding.CompiledChain.Add(UE::Chooser::FCompiledBindingElement(Function));
				StructType = CastField<FObjectProperty>(Function->GetReturnProperty())->PropertyClass;
			}
		}

		if (!bFound)
		{
#if WITH_EDITORONLY_DATA
			CompileMessage = FText::Format(LOCTEXT("Property Not Found", "Property/Function: {0} not Found on Class/Struct: {1}"), FText::FromName(PropertyBindingChain[PropertyChainIndex]), StructType->GetDisplayNameText());
#endif
			UE_ASSET_LOG(LogChooser, Error, Owner->GetContextOwnerAsset(), TEXT("Property/Function: %s not Found on Class/Struct %s"), *PropertyBindingChain[PropertyChainIndex].ToString(), *StructType->GetName());
			CompiledBinding = nullptr;
			return;
		}
	}

	bool bFound = false;
	if (PropertyBindingChain.IsEmpty() && IsBoundToRoot)
	{
		// handle binding directly to a context struct
		OutCompiledBinding.CompiledChain.Add(UE::Chooser::FCompiledBindingElement(0));
		
		if (const FContextObjectTypeStruct* StructContext = ContextData[ContextIndex].GetPtr<FContextObjectTypeStruct>())
		{
			OutCompiledBinding.StructType = StructContext->Struct;
		}
		bFound = true;
	}
	else
	{
#if WITH_EDITOR
		Owner->AddCompileDependency(StructType);
		OutCompiledBinding.Dependencies.AddUnique(StructType);
#endif

		if (const FProperty* BaseProperty = FindFProperty<FProperty>(StructType, PropertyBindingChain.Last()))
		{
			bFound = true;
			
			// last element should be the actual property - add it's offset to whatever was accumulated from struct offsets
			CurrentOffset += BaseProperty->GetOffset_ForInternal();
			OutCompiledBinding.CompiledChain.Add(UE::Chooser::FCompiledBindingElement(CurrentOffset));
			
			if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(BaseProperty))
			{
				OutCompiledBinding.CompiledChain.Last().Mask = BoolProperty->GetFieldMask();
				OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Bool;
			}
			if (BaseProperty->IsA<FFloatProperty>())
			{
				OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Float;
			}
			else if (BaseProperty->IsA<FDoubleProperty>())
			{
				OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Double;
			}
			else if (BaseProperty->IsA<FIntProperty>())
			{
				OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Int32;
			}
			else if (BaseProperty->IsA<FSoftObjectProperty>())
			{
				OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::SoftObjectRef;
			}
			else if (BaseProperty->IsA<FStructProperty>())
            {
            	OutCompiledBinding.StructType = CastField<FStructProperty>(BaseProperty)->Struct;
            }
		}
		else
		{
			// handle function calls 
			if (const UClass* ClassType = Cast<const UClass>(StructType))
			{
				if (UFunction* Function = ClassType->FindFunctionByName(PropertyBindingChain.Last()))
				{
					bFound = true;
					
					const FProperty* ReturnProperty = Function->GetReturnProperty();
					OutCompiledBinding.CompiledChain.Add(UE::Chooser::FCompiledBindingElement(Function));
					if (ReturnProperty->IsA<FFloatProperty>())
					{
						OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Float;
					}
					else if (ReturnProperty->IsA<FDoubleProperty>())
					{
						OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Double;
					}
					else if (ReturnProperty->IsA<FIntProperty>())
					{
						OutCompiledBinding.PropertyType = UE::Chooser::EChooserPropertyAccessType::Int32;
					}
				}
			}
		}
	}

	if (bFound)
	{
		FScopeLock Lock(&CompiledBindingCacheLock);
#if WITH_EDITORONLY_DATA
		NewCompiledBinding->SerialNumber = CompiledBindingSerialNumber;
#endif
		CompiledBindingCache.Add(Id, NewCompiledBinding);
		CompiledBinding = NewCompiledBinding;
	}
	else
	{
#if WITH_EDITORONLY_DATA
		CompileMessage = FText::Format(LOCTEXT("Property Not Found", "Property/Function: {0} not Found on Class/Struct: {1}"), FText::FromName(PropertyBindingChain.Last()), StructType->GetDisplayNameText());
#endif
		UE_ASSET_LOG(LogChooser, Error, Owner->GetContextOwnerAsset(), TEXT("Property/Function: %s not Found on Class/Struct %s"), *PropertyBindingChain.Last().ToString(), *StructType->GetName());
		CompiledBinding = nullptr;
 	}
}

#if WITH_EDITORONLY_DATA
void FChooserEnumPropertyBinding::SetPropertyData(const IHasContextClass* HasContext, FField* Property)
{
	if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property))
	{
		Enum = EnumProperty->GetEnum();
	}
	else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(Property))
	{
		Enum = ByteProperty->Enum;
	}
}

void FChooserObjectPropertyBinding::SetPropertyData(const IHasContextClass* HasContext, FField* Property)
{
	if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
	{
		AllowedClass = ObjectProperty->PropertyClass;
	}
}

void FChooserStructPropertyBinding::SetPropertyData(const IHasContextClass* HasContext, FField* Property)
{
	StructType = nullptr;

	if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
	{
		StructType = StructProperty->Struct;
	}
	else if (this->IsBoundToRoot)
	{
		// direct binding to a context struct
		if (HasContext)
		{
			TConstArrayView<FInstancedStruct> ContextData = HasContext->GetContextData();
			if (ContextData.IsValidIndex(ContextIndex))
			{
				if (const FContextObjectTypeStruct* StructContext = ContextData[ContextIndex].GetPtr<FContextObjectTypeStruct>())
				{
					StructType = StructContext->Struct;
				}
			}
		}
	}
}
#endif


namespace UE::Chooser
{
	
	void RuntimeValidateContext(const UObject* Chooser, const TArray<FInstancedStruct>& ContextData, FChooserEvaluationContext& Context)
	{
		if (!CVarEnableDetailedWarnings.GetValueOnAnyThread())
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(ValidateChooserContext);
		int ContextNum = ContextData.Num();
		for (int i = 0; i < ContextNum; i++)
		{
			const FContextObjectTypeClass* ExpectedClassType = ContextData[i].GetPtr<FContextObjectTypeClass>();
			const FContextObjectTypeStruct* ExpectedStructType = ContextData[i].GetPtr<FContextObjectTypeStruct>();

			if (ExpectedClassType)
			{
				if (ExpectedClassType->Class)
				{
					if (Context.Params.IsValidIndex(i) && Context.Params[i].IsValid())
					{
						if (FChooserEvaluationInputObject* InputObjectParam = Context.Params[i].GetPtr<FChooserEvaluationInputObject>())
						{
							if (InputObjectParam->Object)
							{
								if (!InputObjectParam->Object->GetClass()->IsChildOf(ExpectedClassType->Class))
								{
									UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an object of type %ls, but an object of type %ls was passed in.",
										ToCStr(Chooser->GetName()), i, ToCStr(ExpectedClassType->Class->GetName()), ToCStr(InputObjectParam->Object->GetClass()->GetName()));
								}
							}
							else
							{
								UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an object of type %ls, but null was passed in.",
									ToCStr(Chooser->GetName()), i, ToCStr(ExpectedClassType->Class->GetName()));
							}

						}
						else
						{
							if (Context.Params[i].GetScriptStruct())
							{
								UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an object of type %ls, but was passed a struct of type %ls.",
									ToCStr(Chooser->GetName()), i, ToCStr(ExpectedClassType->Class->GetName()), ToCStr(Context.Params[i].GetScriptStruct()->GetName()));
							}
							else
							{
								
								UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an object of type %ls, but was passed a null struct param",
									ToCStr(Chooser->GetName()), i, ToCStr(ExpectedClassType->Class->GetName()));
							}
						}
					}
					else
					{
						UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an object of type %ls, but nothing was passed in.",
							ToCStr(Chooser->GetName()), i, ToCStr(ExpectedClassType->Class->GetName()));
					}
				}
			}
			else if (ExpectedStructType)
			{
				if (ExpectedStructType->Struct)
				{
					if (Context.Params.IsValidIndex(i) && Context.Params[i].IsValid())
					{
						if (FChooserEvaluationInputObject* InputObjectParam = Context.Params[i].GetPtr<FChooserEvaluationInputObject>())
						{
							if (InputObjectParam->Object)
							{
								UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an struct of type %ls, but was passed a object of type %ls.",
									ToCStr(Chooser->GetName()), i, ToCStr(ExpectedStructType->Struct->GetName()), ToCStr(InputObjectParam->Object->GetClass()->GetName()));
							}
							else
							{
								UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an struct of type %ls, but was passed a null object.",
									ToCStr(Chooser->GetName()), i, ToCStr(ExpectedStructType->Struct->GetName()));
							}
						}
						else
						{
							if (Context.Params[i].GetScriptStruct() != ExpectedStructType->Struct)
							{
								if (Context.Params[i].GetScriptStruct())
								{
									UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an struct of type %ls, but was passed a struct of type %ls.",
										ToCStr(Chooser->GetName()), i, ToCStr(ExpectedStructType->Struct->GetName()), ToCStr(Context.Params[i].GetScriptStruct()->GetName()));
								}
								else
								{
									UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an struct of type %ls, but was passed a null struct parameter.",
										ToCStr(Chooser->GetName()), i, ToCStr(ExpectedStructType->Struct->GetName()));
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogChooser, Error, "Chooser Table: %ls ContextData entry %d expects an struct of type %ls, but nothing was passed in.",
							ToCStr(Chooser->GetName()), i, ToCStr(ExpectedStructType->Struct->GetName()));
					}
				}
			}
			else
			{
				// UAF has it's own child struct, so since this code can't be aware of that type, for now disabling this error
				//UE_LOGF(LogChooser, Error, "Chooser Table: %ls  ContextData entry %i is of unknown type or none.", ToCStr(Chooser->GetName()), i);
			}
		}
	}


	bool ResolveCompiledPropertyChain(FChooserEvaluationContext& Context, const FChooserPropertyBinding& Binding, FResolvedPropertyChainResult& Result)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Chooser_ResolveCompiledPropertyChain);

		const UStruct* InputType = nullptr;
		uint8* Container = nullptr;
		
		if (!Binding.CompiledBinding.IsValid())
		{
			return false;
		}
		
		const FCompiledBinding& CompiledBinding = *Binding.CompiledBinding.Get();

		if(!Context.Params.IsValidIndex(CompiledBinding.ContextIndex))
		{
			UE_LOGF(LogChooser, Error, "Invalid Index {%d} while resolving compiled property chain.", CompiledBinding.ContextIndex);
			return false;
		}
		
		if (FChooserEvaluationInputObject* ObjectInput = Context.Params[CompiledBinding.ContextIndex].GetPtr<FChooserEvaluationInputObject>())
		{
			UObject* Object = ObjectInput->Object.Get();
			Container = reinterpret_cast<uint8*>(Object);
			if (Object)
			{
				InputType = Object->GetClass();
			}
		}
		else
		{
			Container = Context.Params[CompiledBinding.ContextIndex].GetMemory();
			InputType = Context.Params[CompiledBinding.ContextIndex].GetScriptStruct();
		}

		if (Container == nullptr || InputType == nullptr)
		{
			return false;
		}

		if (!InputType->IsChildOf(CompiledBinding.TargetType))
		{
			UE_LOGF(LogChooser, Error, "Property Binding compiled for type: {%ls} is being evaluated on incompatible type: {%ls}.", ToCStr(CompiledBinding.TargetType->GetName()), ToCStr(InputType->GetName()));
			return false;
		}
		
		for (int i = 0; i<CompiledBinding.CompiledChain.Num() - 1; i++)
		{
			if (Container)
			{
				const FCompiledBindingElement& Element = CompiledBinding.CompiledChain[i];
				if (!Element.bIsFunction)
				{
					Container = *reinterpret_cast<uint8**>(Container + CompiledBinding.CompiledChain[i].Offset);
				}
				else
				{
					UObject* Object = reinterpret_cast<UObject*>(Container);
					if (Element.Function->IsNative())
					{
						FFrame Stack(Object, Element.Function, nullptr, nullptr, Element.Function->ChildProperties);
						Element.Function->Invoke(Object, Stack, &Container);
					}
					else
					{
						Object->ProcessEvent(Element.Function, &Container);
					}
				}
			}
		}

		Result.Container = Container;
		const FCompiledBindingElement& Last = CompiledBinding.CompiledChain.Last();
		if (Last.bIsFunction)
		{
			Result.Function = Last.Function;
		}
		else
		{
			Result.PropertyOffset = Last.Offset;
			Result.Mask = Last.Mask;
		}
		Result.PropertyType = CompiledBinding.PropertyType;
		Result.StructType = CompiledBinding.StructType; 
		return true;
	}
	
	bool ResolvePropertyChain(uint8* Container, const UStruct*& StructType, const FChooserPropertyBinding& PropertyBinding, FResolvedPropertyChainResult& Result)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Chooser_ResolvePropertyChain);

		if (PropertyBinding.PropertyBindingChain.Num() == 0)
		{
			if (PropertyBinding.IsBoundToRoot)
			{
				Result.Container = Container;
				Result.StructType = StructType;
				return true;
			}
			else
			{
				return false;
			}
		}
	
		const int PropertyChainLength = PropertyBinding.PropertyBindingChain.Num();
		for(int PropertyChainIndex = 0; PropertyChainIndex < PropertyChainLength - 1; PropertyChainIndex++)
		{
			if (const FStructProperty* StructProperty = FindFProperty<FStructProperty>(StructType, PropertyBinding.PropertyBindingChain[PropertyChainIndex]))
			{
				StructType = StructProperty->Struct;
				Container = StructProperty->ContainerPtrToValuePtr<uint8>(Container);
			}
			else if (const FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(StructType, PropertyBinding.PropertyBindingChain[PropertyChainIndex]))
			{
				StructType = ObjectProperty->PropertyClass;
				Container = static_cast<uint8*>(*ObjectProperty->ContainerPtrToValuePtr<TObjectPtr<UObject>>(Container));
				if (Container == nullptr)
				{
					return false;
				}
			}
			else
			{
				// check if it's a member function
				if (const UClass* ClassType = Cast<const UClass>(StructType))
				{
					if (UFunction* Function = ClassType->FindFunctionByName(PropertyBinding.PropertyBindingChain[PropertyChainIndex]))
					{
						UObject* Object = reinterpret_cast<UObject*>(Container);
						if (Function->IsNative())
						{
							FFrame Stack(Object, Function, nullptr, nullptr, Function->ChildProperties);
							Function->Invoke(Object, Stack, &Container);
						}
						else
						{
							Object->ProcessEvent(Function, &Container);
						}
						
						if (Container == nullptr)
						{
							return false;
						}
						else
						{
							StructType = reinterpret_cast<UObject*>(Container)->GetClass();
						}
					}
					else
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}
		}


		bool bFound = false;

		if (const FProperty* BaseProperty = FindFProperty<FProperty>(StructType, PropertyBinding.PropertyBindingChain.Last()))
		{
			bFound = true;

			Result.Container = Container;
			Result.PropertyOffset = BaseProperty->GetOffset_ForInternal();
			
			if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(BaseProperty))
			{
				Result.Mask = BoolProperty->GetFieldMask();
				Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Bool;
			}
			if (BaseProperty->IsA<FFloatProperty>())
			{
				Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Float;
			}
			else if (BaseProperty->IsA<FDoubleProperty>())
			{
				Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Double;
			}
			else if (BaseProperty->IsA<FIntProperty>())
			{
				Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Int32;
			}
			else if (BaseProperty->IsA<FSoftObjectProperty>())
			{
				Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::SoftObjectRef;
			}
			else if (BaseProperty->IsA<FStructProperty>())
			{
				Result.StructType = CastField<FStructProperty>(BaseProperty)->Struct;
			}
		}
		else
		{
			// handle function calls 
			if (const UClass* ClassType = Cast<const UClass>(StructType))
			{
				if (UFunction* Function = ClassType->FindFunctionByName(PropertyBinding.PropertyBindingChain.Last()))
				{
					bFound = true;
					
					Result.Container = Container;
					Result.Function = Function;
					
					const FProperty* ReturnProperty = Function->GetReturnProperty();
					if (ReturnProperty->IsA<FFloatProperty>())
					{
						Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Float;
					}
					else if (ReturnProperty->IsA<FDoubleProperty>())
					{
						Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Double;
					}
					else if (ReturnProperty->IsA<FIntProperty>())
					{
						Result.PropertyType = UE::Chooser::EChooserPropertyAccessType::Int32;
					}
				}
			}
		}
	
		return bFound;
	}

	bool ResolvePropertyChain(FChooserEvaluationContext& Context, const FChooserPropertyBinding& PropertyBinding, FResolvedPropertyChainResult& Result)
	{
		bool bUseCompiledChain = PropertyBinding.CompiledBinding.IsValid();

#if WITH_EDITOR
		if (!CVarUseCompiledPropertyChainsInEditor.GetValueOnAnyThread())
		{
			bUseCompiledChain = false;
		}
#endif
		
		if (bUseCompiledChain)
		{
			return ResolveCompiledPropertyChain(Context,PropertyBinding,Result);
		}
		
		if (Context.Params.IsValidIndex(PropertyBinding.ContextIndex))
		{
			uint8* Container = nullptr;
			const UStruct* StructType = nullptr;
			if (FChooserEvaluationInputObject* ObjectParam = Context.Params[PropertyBinding.ContextIndex].GetPtr<FChooserEvaluationInputObject>())
			{
				Container = static_cast<uint8*>(ObjectParam->Object);
				if (Container)
				{
					StructType = ObjectParam->Object->GetClass();
				}
			}
			else
			{
				Container = Context.Params[PropertyBinding.ContextIndex].GetMemory();
				StructType = Context.Params[PropertyBinding.ContextIndex].GetScriptStruct();
			}


			if (Container == nullptr || StructType == nullptr)
			{
				return false;
			}

			return ResolvePropertyChain(Container, StructType, PropertyBinding, Result);
		}

		return false;
	}

	
#if WITH_EDITOR
	void CopyPropertyChain(const TArray<FBindingChainElement>& InBindingChain, FChooserPropertyBinding& OutPropertyBinding)
	{
		OutPropertyBinding.PropertyBindingChain.Empty();

		if (InBindingChain.Num() == 0)
		{
			OutPropertyBinding.ContextIndex = -1;
		}
		else
		{
			OutPropertyBinding.ContextIndex = InBindingChain[0].ArrayIndex;
		}

		for (int32 i = 1; i < InBindingChain.Num(); ++i)
		{
			OutPropertyBinding.PropertyBindingChain.Emplace(InBindingChain[i].Field.GetFName());
		}

		OutPropertyBinding.IsBoundToRoot = (InBindingChain.Num()==1);
	}
#endif

	
}

#undef LOCTEXT_NAMESPACE
