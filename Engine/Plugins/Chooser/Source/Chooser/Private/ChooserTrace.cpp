// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTrace.h"

#include "IChooserColumn.h"
#include "ObjectTrace.h"
#include "IObjectChooser.h"
#include "Serialization/BufferArchive.h"

#if CHOOSER_TRACE_ENABLED

UE_TRACE_CHANNEL(ChooserChannel)

UE_TRACE_EVENT_BEGIN(Chooser, ChooserEvaluation2)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ChooserId)
	UE_TRACE_EVENT_FIELD(uint64, OwnerId)
	UE_TRACE_EVENT_FIELD(int32[], SelectedIndices)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Chooser, ChooserValue)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, ChooserId)
	UE_TRACE_EVENT_FIELD(uint64, OwnerId)
	UE_TRACE_EVENT_FIELD(uint8[], Value)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Key)
UE_TRACE_EVENT_END()

namespace
{
	UObject* GetContextObject(const FChooserEvaluationContext& Context)
	{
		for (const FStructView& ContextEntry : Context.Params)
		{
			if (const FChooserEvaluationInputObject* ContextObjectInput = ContextEntry.GetPtr<FChooserEvaluationInputObject>())
			{
				return ContextObjectInput->Object;
			}
		}
		return nullptr;
	}
}



void FChooserTrace::OutputChooserValueArchive(const FChooserEvaluationContext& Context, const TCHAR* InKey, const FBufferArchive& InValueArchive)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(ChooserChannel);

	if (!bChannelEnabled  || Context.DebuggingInfo.CurrentChooser == nullptr)
	{
		return;
	}

	const UObject* ContextObject = GetContextObject(Context);

	if (ContextObject == nullptr || CANNOT_TRACE_OBJECT(ContextObject->GetWorld()))
	{
		return;
	}
	
	TRACE_OBJECT(Context.DebuggingInfo.CurrentChooser);
	TRACE_OBJECT(ContextObject);

	UE_TRACE_LOG(Chooser, ChooserValue, ChooserChannel)
		<< ChooserValue.RecordingTime(FObjectTrace::GetWorldElapsedTime(ContextObject->GetWorld()))
    	<< ChooserValue.ChooserId(FObjectTrace::GetObjectId(Context.DebuggingInfo.CurrentChooser))
    	<< ChooserValue.OwnerId(FObjectTrace::GetObjectId(ContextObject))
    	<< ChooserValue.Key(InKey)
    	<< ChooserValue.Value(InValueArchive.GetData(), InValueArchive.Num());
}

void FChooserTrace::OutputChooserEvaluation(const UObject* InChooser, const FChooserEvaluationContext& Context, int32 Index)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(ChooserChannel);
	if (!bChannelEnabled || InChooser == nullptr)
	{
		return;
	}
	
	const UObject* ContextObject = GetContextObject(Context);

	if (ContextObject == nullptr || CANNOT_TRACE_OBJECT(ContextObject->GetWorld()))
	{
		return;
	}

	TRACE_OBJECT(InChooser);
	TRACE_OBJECT(ContextObject);

	UE_TRACE_LOG(Chooser, ChooserEvaluation2, ChooserChannel)
	<< ChooserEvaluation2.RecordingTime(FObjectTrace::GetWorldElapsedTime(ContextObject->GetWorld()))
	<< ChooserEvaluation2.ChooserId(FObjectTrace::GetObjectId(InChooser))
	<< ChooserEvaluation2.OwnerId(FObjectTrace::GetObjectId(ContextObject))
	<< ChooserEvaluation2.SelectedIndices(&Index, 1);
}

void FChooserTrace::OutputChooserEvaluation(const UObject* InChooser, const FChooserEvaluationContext& Context, const FChooserIndexArray& SelectedIndexArray)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(ChooserChannel);
	if (!bChannelEnabled || InChooser == nullptr)
	{
		return;
	}
	
	const UObject* ContextObject = GetContextObject(Context);

	if (ContextObject == nullptr || CANNOT_TRACE_OBJECT(ContextObject->GetWorld()))
	{
		return;
	}

	TRACE_OBJECT(InChooser);
	TRACE_OBJECT(ContextObject);

	TArray<int32> SelectedIndices;
	SelectedIndices.Reserve(FMath::Max(SelectedIndexArray.Num(), 1u));
	if (SelectedIndexArray.Num() == 0)
	{
		SelectedIndices.Add(ChooserColumn_SpecialIndex_Fallback);
	}
	else
	{
		for(const FChooserIndexArray::FIndexData& IndexEntry : SelectedIndexArray)
		{
			SelectedIndices.Add(IndexEntry.Index);		
		}
	}
	

	UE_TRACE_LOG(Chooser, ChooserEvaluation2, ChooserChannel)
	<< ChooserEvaluation2.RecordingTime(FObjectTrace::GetWorldElapsedTime(ContextObject->GetWorld()))
	<< ChooserEvaluation2.ChooserId(FObjectTrace::GetObjectId(InChooser))
	<< ChooserEvaluation2.OwnerId(FObjectTrace::GetObjectId(ContextObject))
	<< ChooserEvaluation2.SelectedIndices(&SelectedIndices[0], SelectedIndices.Num());
}

#endif

