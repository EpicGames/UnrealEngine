// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariableParameters.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConsoleVariableParameters)

bool FBoolCVarProperty::GetValue(FChooserEvaluationContext& Context, bool& OutResult) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
 	if (Variable)
 	{
 		OutResult = Variable->GetBool();
 		return true;
 	}
 	
 	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
 	return false;
}

bool FBoolCVarProperty::SetValue(FChooserEvaluationContext& Context, bool InValue) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Variable->Set(InValue);
		return true;
	}
	
	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
	return false;
}

void FBoolCVarProperty::GetDisplayName(FText& OutName) const
{
	OutName = FText::FromString(VariableName);
}

bool FFloatCVarProperty::GetValue(FChooserEvaluationContext& Context, double& OutResult) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		OutResult = Variable->GetFloat();
		return true;
	}
	
	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
	return false;
}

bool FFloatCVarProperty::SetValue(FChooserEvaluationContext& Context, double InValue) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Variable->Set(static_cast<float>(InValue));
		return true;
	}
	
	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
	return false;
}

void FFloatCVarProperty::GetDisplayName(FText& OutName) const
{
	OutName = FText::FromString(VariableName);
}

bool FEnumCVarProperty::GetValue(FChooserEvaluationContext& Context, uint8& OutResult) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		OutResult = Variable->GetInt();
		return true;
	}
	
	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
	return false;
}

bool FEnumCVarProperty::SetValue(FChooserEvaluationContext& Context, uint8 InValue) const
{
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Variable->Set(InValue);
		return true;
	}
	
	UE_LOGF(LogChooser, Warning, "Failed to find console variable '%ls'.", *VariableName);
	return false;
}

void FEnumCVarProperty::GetDisplayName(FText& OutName) const
{
	OutName = FText::FromString(VariableName);
}
