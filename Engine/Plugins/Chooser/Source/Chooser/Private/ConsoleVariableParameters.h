// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterBool.h"
#include "IChooserParameterFloat.h"
#include "IChooserParameterEnum.h"
#include "ConsoleVariableParameters.generated.h"


USTRUCT(DisplayName = "Bool CVar Param")
struct FBoolCVarProperty :  public FChooserParameterBoolBase
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category="Variable")
	FString VariableName;

	virtual bool GetValue(FChooserEvaluationContext& Context, bool& OutResult) const override;
	virtual bool SetValue(FChooserEvaluationContext& Context, bool InValue) const override;
	virtual void GetDisplayName(FText& OutName) const override;
};

USTRUCT(DisplayName = "Float CVar Param")
struct FFloatCVarProperty :  public FChooserParameterFloatBase
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category="Variable")
	FString VariableName;

	virtual bool GetValue(FChooserEvaluationContext& Context, double& OutResult) const override;
	virtual bool SetValue(FChooserEvaluationContext& Context, double InValue) const override;
	virtual void GetDisplayName(FText& OutName) const override;
};

USTRUCT(DisplayName = "Enum CVar Param")
struct FEnumCVarProperty :  public FChooserParameterEnumBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category="Variable")
	FString VariableName;

	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UEnum> Enum;

	virtual bool GetValue(FChooserEvaluationContext& Context, uint8& OutResult) const override;
	virtual bool SetValue(FChooserEvaluationContext& Context, uint8 InValue) const override;

	virtual void GetDisplayName(FText& OutName) const override;

	#if WITH_EDITOR
	virtual const UEnum* GetEnum() const override { return Enum; }
	#endif
};