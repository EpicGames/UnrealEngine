// Copyright Epic Games, Inc. All Rights Reserved.
#include "EnumColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#include "Engine/UserDefinedEnum.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnumColumn)

bool FEnumContextProperty::GetValue(FChooserEvaluationContext& Context, uint8& OutResult) const
{
	return Binding.GetValue(Context, OutResult);
}

bool FEnumContextProperty::SetValue(FChooserEvaluationContext& Context, uint8 InValue) const
{
	return Binding.SetValue(Context, InValue);
}

FEnumColumn::FEnumColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FEnumContextProperty::StaticStruct());
#endif
}

void FEnumColumn::PostLoad()
{
	Super::PostLoad();
	
	if (InputValue.IsValid())
	{
		InputValue.GetMutable<FChooserParameterBase>().PostLoad();
	}

#if WITH_EDITORONLY_DATA
	const UEnum* Enum = nullptr;
	if (InputValue.IsValid())
	{
		Enum = InputValue.Get<FChooserParameterEnumBase>().GetEnum();
	}
	
	for(FChooserEnumRowData& CellData : RowValues)
	{
		// upgrade data for "Any" support
		if (CellData.CompareNotEqual_DEPRECATED)
		{
			CellData.CompareNotEqual_DEPRECATED = false;
			CellData.Comparison = EEnumColumnCellValueComparison::MatchNotEqual;
		}

		if (Enum)
		{
			if (Enum->IsValidEnumName(CellData.ValueName))
			{
				CellData.Value = Enum->GetValueByName(CellData.ValueName);
			}
			else
			// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
			{
				CellData.ValueName = Enum->GetNameByValue(CellData.Value);
			}
		}
	}
#endif
}

bool FChooserEnumRowData::Evaluate(const uint8 LeftHandSide) const
{
	switch (Comparison)
	{
		case EEnumColumnCellValueComparison::MatchEqual:
			return LeftHandSide == Value;

		case EEnumColumnCellValueComparison::MatchNotEqual:
			return LeftHandSide != Value;

		case EEnumColumnCellValueComparison::MatchAny:
			return true;

		default:
			return false;
	}
}

void FEnumColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	uint8 Result = 0;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterEnumBase>().GetValue(Context, Result))
	{
#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif

		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result);
		
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValues.IsValidIndex(IndexData.Index))
			{
				const FChooserEnumRowData& RowValue = RowValues[IndexData.Index];
				if (RowValue.Evaluate(Result))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FEnumColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);

		// make another property bag in place of our struct, just so that the value enum will be correctly typed in the details panel
		FInstancedPropertyBag Struct;
		Struct.AddProperty("Value", EPropertyBagPropertyType::Enum, const_cast<UEnum*>(InputValue.Get<FChooserParameterEnumBase>().GetEnum()));
		Struct.SetValueEnum("Value", RowValues[RowIndex].Value, InputValue.Get<FChooserParameterEnumBase>().GetEnum());
		Struct.AddProperty("Comparison", EPropertyBagPropertyType::Enum, StaticEnum<EEnumColumnCellValueComparison>());
		Struct.SetValueEnum("Comparison", static_cast<uint8>(RowValues[RowIndex].Comparison), StaticEnum<EEnumColumnCellValueComparison>());
		
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, const_cast<UPropertyBag*>(Struct.GetPropertyBagStruct()));
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, Struct.GetValue());
	}
	
	void FEnumColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData",ColumnIndex);

		FInstancedPropertyBag Struct;
		Struct.AddProperty("Value", EPropertyBagPropertyType::Enum, InputValue.Get<FChooserParameterEnumBase>().GetEnum());
		Struct.AddProperty("Comparison", EPropertyBagPropertyType::Enum, StaticEnum<EEnumColumnCellValueComparison>());
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, const_cast<UPropertyBag*>(Struct.GetPropertyBagStruct()));
		if (FStructView* StructView = Result.TryGetValue())
		{
			const UScriptStruct* StructDefinition = StructView->GetScriptStruct();
			if (const FEnumProperty* ValueEnumProperty = CastField<FEnumProperty>(StructDefinition->FindPropertyByName("Value")))
			{
				ValueEnumProperty->GetValue_InContainer(StructView->GetMemory(), &RowValues[RowIndex].Value);
			}
			else if (const FByteProperty* ValueByteProperty = CastField<FByteProperty>(StructDefinition->FindPropertyByName("Value")))
            {
            	ValueByteProperty->GetValue_InContainer(StructView->GetMemory(), &RowValues[RowIndex].Value);
            }
			
			if (const FEnumProperty* ComparisonProperty = CastField<FEnumProperty>(StructDefinition->FindPropertyByName("Comparison")))
			{
				ComparisonProperty->GetValue_InContainer(StructView->GetMemory(), &RowValues[RowIndex].Comparison);
			}
		}
	}

	void FEnumColumn::EnumChanged(const UEnum* Enum)
	{
		if (InputValue.IsValid())
		{
			if (Enum == InputValue.Get<FChooserParameterEnumBase>().GetEnum())
			{
				for(FChooserEnumRowData& CellData : RowValues)
				{
					if (Enum->IsValidEnumName(CellData.ValueName))
					{
						CellData.Value = Enum->GetValueByName(CellData.ValueName);
					}
					else
					// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
					{
						CellData.ValueName = Enum->GetNameByValue(CellData.Value);
					}
				}
			}
		}
	}
#endif
