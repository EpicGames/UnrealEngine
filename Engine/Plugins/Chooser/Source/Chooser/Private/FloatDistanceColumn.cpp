// Copyright Epic Games, Inc. All Rights Reserved.
#include "FloatDistanceColumn.h"
#include "FloatRangeColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatDistanceColumn)

FFloatDistanceColumn::FFloatDistanceColumn()
{
	InputValue.InitializeAs(FFloatContextProperty::StaticStruct());
}

#if WITH_EDITOR
void FFloatDistanceColumn::AutoPopulate(int32 RowIndex, UObject* OutputObject)
{
	if (AutoPopulator)
	{
		if (RowValues.IsValidIndex(RowIndex) && !RowValues[RowIndex].DisableAutoPopulate) 
		{
			float Result = 0.f;
			bool Success = false;
			static_cast<UFloatAutoPopulator*>(AutoPopulator->GetDefaultObject())->NativeAutoPopulate(OutputObject, Success, Result);
			if (Success)
			{
				RowValues[RowIndex].Value = Result;
			}
		}
	}
}

bool FFloatDistanceColumn::EditorTestFilter(int32 RowIndex) const
{
	if (!bFilterOverMaxDistance)
	{
		// filtering not enabled, only scoring
		return true;
	}
	
	if (RowValues.IsValidIndex(RowIndex))
	{
		const FChooserFloatDistanceRowData& RowValue = RowValues[RowIndex];
		
		if (bWrapInput)
		{
			double WrappedValue = FMath::Wrap(TestValue, MinValue, MaxValue);
			double Distance =  fabs(WrappedValue - RowValue.Value);
			if (Distance > 0.5 * MaxValue - MinValue)
			{
				Distance = MaxValue - MinValue - Distance;
			}
			return Distance <= MaxDistance;
		}
		else
		{
			return fabs(TestValue - RowValue.Value) <= MaxDistance;
		}
	}
	
	return false;
}


float FFloatDistanceColumn::EditorTestCost(int32 RowIndex) const
{
	if (RowValues.IsValidIndex(RowIndex))
	{
		const FChooserFloatDistanceRowData& RowValue = RowValues[RowIndex];
		
		if (bWrapInput)
		{
			double WrappedValue = FMath::Wrap(TestValue, MinValue, MaxValue);
			double Distance =  fabs(WrappedValue - RowValue.Value);
			if (Distance > 0.5 * MaxValue - MinValue)
			{
				Distance = MaxValue - MinValue - Distance;
			}
			return FMath::Min(Distance / MaxDistance, 1.f);
		}
		else
		{
			return FMath::Min(fabs(TestValue - RowValue.Value) / MaxDistance, 1.f);
		}
	}
	
	return 0.0f;
}

#endif

void FFloatDistanceColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	double Result = 0.0f;
	if (InputValue.IsValid() && 
		InputValue.Get<FChooserParameterFloatBase>().GetValue(Context, Result))
	{	
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result);

#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif
	
		if (bWrapInput)
		{
			double WrappedValue = FMath::Wrap(Result, MinValue, MaxValue);
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (RowValues.Num() > static_cast<int>(IndexData.Index))
				{
					const FChooserFloatDistanceRowData& RowValue = RowValues[IndexData.Index];
					
					double Distance =  fabs(WrappedValue - RowValue.Value);
					if (Distance > 0.5 * MaxValue - MinValue)
					{
						Distance = MaxValue - MinValue - Distance;
					}
					
					if (!bFilterOverMaxDistance || Distance < MaxDistance)
					{
						float Cost = FMath::Min(Distance / MaxDistance, 1.0);
						IndexListOut.Push({IndexData.Index, IndexData.Cost + CostMultiplier * Cost});
					}
				}
			}
		}
		else
		{
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (RowValues.IsValidIndex(IndexData.Index))
				{
					const FChooserFloatDistanceRowData& RowValue = RowValues[IndexData.Index];
					double Distance = fabs(Result - RowValue.Value);
					if (!bFilterOverMaxDistance || Distance < MaxDistance)
					{
						float Cost = FMath::Min(Distance / MaxDistance, 1.0);
						IndexListOut.Push({IndexData.Index, IndexData.Cost + CostMultiplier * Cost});
					}
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FFloatDistanceColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterFloatBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData", ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserFloatDistanceRowData::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FFloatDistanceColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
   		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserFloatDistanceRowData::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FChooserFloatDistanceRowData>();
		}
	}
#endif
