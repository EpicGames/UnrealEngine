// Copyright Epic Games, Inc. All Rights Reserved.
#include "FloatRangeColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatRangeColumn)

bool FFloatContextProperty::SetValue(FChooserEvaluationContext& Context, double InValue) const
{
	return Binding.SetValue(Context, InValue);
}

bool FFloatContextProperty::GetValue(FChooserEvaluationContext& Context, double& OutResult) const
{
	return Binding.GetValue(Context, OutResult);
}

FFloatRangeColumn::FFloatRangeColumn()
{
	InputValue.InitializeAs(FFloatContextProperty::StaticStruct());
}

#if WITH_EDITOR
bool FFloatRangeColumn::EditorTestFilter(int32 RowIndex) const
{
	if (RowValues.IsValidIndex(RowIndex))
	{
		const FChooserFloatRangeRowData& RowValue = RowValues[RowIndex];
		
		if (bWrapInput)
		{
			double TestResult = FMath::Wrap(TestValue, MinValue, MaxValue);
			if (RowValue.Max < RowValue.Min) // eg for an angle range from  135 to -135  (135 to 180 or -180 to -135)
			{
				return (RowValue.bNoMin || TestResult >= RowValue.Min) || (RowValue.bNoMax || TestResult <= RowValue.Max);
			}
			else
			{
				return (RowValue.bNoMin || TestResult >= RowValue.Min) && (RowValue.bNoMax || TestResult <= RowValue.Max);
			}
		}
		else
		{
			return (RowValue.bNoMin || TestValue >= RowValue.Min) && (RowValue.bNoMax || TestValue <= RowValue.Max);
		}
	}
	return false;
}
#endif

void FFloatRangeColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	double Result = 0.0f;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterFloatBase>().GetValue(Context, Result))
	{		
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result);

#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif

		if (bWrapInput)
		{
			Result = FMath::Wrap(Result, MinValue, MaxValue);
			
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (RowValues.Num() > static_cast<int>(IndexData.Index))
				{
					const FChooserFloatRangeRowData& RowValue = RowValues[IndexData.Index];
					if (RowValue.Max < RowValue.Min) // eg for an angle range from  135 to -135  (135 to 180 or -180 to -135)
					{
						if ((RowValue.bNoMin || Result >= RowValue.Min) || (RowValue.bNoMax || Result <= RowValue.Max))
						{
							IndexListOut.Push(IndexData);
						}
					}
					else if ((RowValue.bNoMin || Result >= RowValue.Min) && (RowValue.bNoMax || Result <= RowValue.Max))
					{
						IndexListOut.Push(IndexData);
					}
				}
			}
		}
		else
		{
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (RowValues.Num() > static_cast<int>(IndexData.Index))
				{
					const FChooserFloatRangeRowData& RowValue = RowValues[IndexData.Index];
					if ((RowValue.bNoMin || Result >= RowValue.Min) && (RowValue.bNoMax || Result <= RowValue.Max))
					{
						IndexListOut.Push(IndexData);
					}
				}
			}
		}
		
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FFloatRangeColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterFloatBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData", ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserFloatRangeRowData::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FFloatRangeColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
   		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserFloatRangeRowData::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FChooserFloatRangeRowData>();
		}
	}
#endif
