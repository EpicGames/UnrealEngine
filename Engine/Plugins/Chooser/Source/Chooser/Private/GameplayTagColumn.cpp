// Copyright Epic Games, Inc. All Rights Reserved.
#include "GameplayTagColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagColumn)


bool FGameplayTagContextProperty::GetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer*& OutResult) const
{
	return Binding.GetValuePtr(Context, OutResult);
}

bool FGameplayTagContextProperty::SetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer& Value) const
{
	return Binding.SetValue(Context, Value);
}

FGameplayTagColumn::FGameplayTagColumn()
{
	InputValue.InitializeAs(FGameplayTagContextProperty::StaticStruct());
}

bool FGameplayTagColumn::TestRow(int32 RowIndex, const FGameplayTagContainer& Value) const
{
	if(RowValues.IsValidIndex(RowIndex))
	{
		bool bPasses = false;	

		if (RowValues[RowIndex].IsEmpty())
		{
			//	An empty container should always pass in inverted mode, but we are going to invert later, so invert here.
			bPasses = !bInvertMatchingLogic;
		}
		else
		{
			const FGameplayTagContainer* A;
			const FGameplayTagContainer* B;
			if (TagMatchDirection == EGameplayTagMatchDirection::RowValueInInput)
			{
				A = &Value;
				B = &RowValues[RowIndex];
			}
			else
			{
				A = &RowValues[RowIndex];
				B = &Value;
			}
			
			
			if (TagMatchType == EGameplayContainerMatchType::All)
			{
				if (bMatchExact ? A->HasAllExact(*B) : A->HasAll(*B))
				{
					bPasses = true;
				}
			}
			else
			{
				if (bMatchExact ? A->HasAnyExact(*B) : A->HasAny(*B))
				{
					bPasses = true;
				}
			}
		}

		if (bInvertMatchingLogic)
		{
			bPasses = !bPasses;
		}

		return bPasses;	
	}
	return false;
}

void FGameplayTagColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	const FGameplayTagContainer* Result = nullptr;
	if (InputValue.IsValid() && InputValue.Get<FChooserParameterGameplayTagBase>().GetValue(Context,Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result->ToString());

#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = *Result;
		}
#endif

		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (TestRow(IndexData.Index, *Result))
			{
				IndexListOut.Push(IndexData);
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FGameplayTagColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterGameplayTagBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FGameplayTagContainer::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FGameplayTagColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FGameplayTagContainer::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FGameplayTagContainer>();
		}
	}	
#endif
