// Copyright Epic Games, Inc. All Rights Reserved.
#include "GameplayTagQueryColumn.h"

#include "ChooserIndexArray.h"
#include "ChooserTrace.h"
#include "GameplayTagColumn.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagQueryColumn)


FGameplayTagQueryColumn::FGameplayTagQueryColumn()
{
	InputValue.InitializeAs(FGameplayTagContextProperty::StaticStruct());
}

bool FGameplayTagQueryColumn::TestRow(int32 RowIndex, const FGameplayTagContainer& Value) const
{
	if (RowValues.IsValidIndex(RowIndex))
	{
		return RowValues[RowIndex].Matches(Value);
	}
	return false;
}

void FGameplayTagQueryColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	const FGameplayTagContainer* Result = nullptr;
	if (InputValue.IsValid() && InputValue.Get<FChooserParameterGameplayTagBase>().GetValue(Context,Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result->ToString());

#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = *Result;
		}
#endif

		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (TestRow(IndexData.Index, *Result))
			{
				IndexListOut.Push(IndexData);
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FGameplayTagQueryColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData", ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Struct, FGameplayTagQuery::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FGameplayTagQueryColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FGameplayTagQuery::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FGameplayTagQuery>();
		}
	}	
#endif
