// Copyright Epic Games, Inc. All Rights Reserved.
#include "MultiEnumColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserTrace.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiEnumColumn)

FMultiEnumColumn::FMultiEnumColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FEnumContextProperty::StaticStruct());
#endif
}

void FMultiEnumColumn::PostLoad()
{
	Super::PostLoad();
	
	if (InputValue.IsValid())
	{
		InputValue.GetMutable<FChooserParameterBase>().PostLoad();
	}
}

void FMultiEnumColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	uint8 Result = 0;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterEnumBase>().GetValue(Context, Result))
	{
#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif

		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result);

		// log if Result > 31
		uint32 ResultBit = 1 << Result;
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValues.IsValidIndex(IndexData.Index))
			{
				const FChooserMultiEnumRowData& RowValue = RowValues[IndexData.Index];
				if (RowValue.Evaluate(ResultBit))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}

#if WITH_EDITOR
	void FMultiEnumColumn::AddToDetails (FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	FText DisplayName;
	InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
	FName PropertyName("RowData",ColumnIndex);
	if (const UEnum* Enum = InputValue.Get<FChooserParameterEnumBase>().GetEnum())
	{
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagContainerType::Array, EPropertyBagPropertyType::Enum, Enum);
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
	
		FPropertyBagArrayRef ArrayRef = PropertyBag.GetMutableArrayRef(PropertyName).GetValue();

		int32 NumEnums = Enum->NumEnums();
		for(int32 i=0; i<NumEnums; i++)
		{
			if (RowValues[RowIndex].Value & (1 << Enum->GetValueByIndex(i)))
			{
				ArrayRef.AddValue();
				ArrayRef.SetValueEnum(ArrayRef.Num()-1,Enum->GetValueByIndex(i), Enum);
			}
		}
	}
}
	
void FMultiEnumColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	FName PropertyName("RowData",ColumnIndex);

	if (const UEnum* Enum = InputValue.Get<FChooserParameterEnumBase>().GetEnum())
	{
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = PropertyBag.GetArrayRef(PropertyName);
		if (const FPropertyBagArrayRef* ArrayRef = Result.TryGetValue())
		{
			RowValues[RowIndex].Value = 0;
			for(int32 i = 0; i<ArrayRef->Num(); i++)
			{
				RowValues[RowIndex].Value  |= (1 << ArrayRef->GetValueEnum(i, Enum).GetValue());
			}
		}
	}
}
#endif
