// Copyright Epic Games, Inc. All Rights Reserved.
#include "NameColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NameColumn)

bool FNameContextProperty::GetValue(FChooserEvaluationContext& Context, FName& OutResult) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			const FName* Name = reinterpret_cast<const FName*>(Result.Container + Result.PropertyOffset);
			OutResult = *Name;
			return true;
		}
	}

	return false;
}

bool FNameContextProperty::SetValue(FChooserEvaluationContext& Context, const FName& OutputName) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			FName* Name = reinterpret_cast<FName*>(Result.Container + Result.PropertyOffset);
			*Name = OutputName;
			return true;
		}
	}

	return false;
}

FChooserNameColumn::FChooserNameColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FNameContextProperty::StaticStruct());
#endif
}

bool FChooserNameRowData::Evaluate(const FName& LeftHandSide) const
{
	switch (Comparison)
	{
		case ENameColumnCellValueComparison::MatchEqual:
			return LeftHandSide == Value;

		case ENameColumnCellValueComparison::MatchNotEqual:
			return LeftHandSide != Value;

		case ENameColumnCellValueComparison::MatchAny:
			return true;

		default:
			return false;
	}
}

void FChooserNameColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	FName Result;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterNameBase>().GetValue(Context, Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result.ToString());
	
#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif
		
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValues.IsValidIndex(IndexData.Index))
			{
				const FChooserNameRowData& RowValue = RowValues[IndexData.Index];
				if (RowValue.Evaluate(Result))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}
#if WITH_EDITOR
	void FChooserNameColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		if (RowValues.IsValidIndex(RowIndex))
		{
			FText DisplayName;
			InputValue.Get<FChooserParameterNameBase>().GetDisplayName(DisplayName);
			FName PropertyName("RowData",ColumnIndex);
			FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserNameRowData::StaticStruct());
			PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
			PropertyBag.AddProperties({PropertyDesc});
			PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
		}
	}

	void FChooserNameColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);

		if (RowValues.IsValidIndex(RowIndex))
		{
			TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserNameRowData::StaticStruct());
			if (FStructView* StructView = Result.TryGetValue())
			{
				RowValues[RowIndex] = StructView->Get<FChooserNameRowData>();
			}
		}
	}
#endif
