// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectChooser_Asset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectChooser_Asset)

UObject* FAssetChooser::ChooseObject(FChooserEvaluationContext& Context) const
{
	return Asset;
}

FObjectChooserBase::EIteratorStatus FAssetChooser::IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return Callback.Execute(Asset);
}

void FSoftAssetChooser::ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const
{
	Result = Asset.ToSoftObjectPath();
}

UObject* FSoftAssetChooser::ChooseObject(FChooserEvaluationContext& Context) const
{
	return Asset.Get();
}

FObjectChooserBase::EIteratorStatus FSoftAssetChooser::IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return Callback.Execute(Asset.LoadSynchronous());
}
