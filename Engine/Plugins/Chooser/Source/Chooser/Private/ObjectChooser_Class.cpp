// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectChooser_Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectChooser_Class)

UObject* FClassChooser::ChooseObject(FChooserEvaluationContext& Context) const
{
	return Class;
}

FObjectChooserBase::EIteratorStatus FClassChooser::IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	return Callback.Execute(Class);
}
