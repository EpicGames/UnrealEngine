// Copyright Epic Games, Inc. All Rights Reserved.
#include "ObjectClassColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectClassColumn)

FObjectClassColumn::FObjectClassColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FObjectContextProperty::StaticStruct());
#endif
}

bool FChooserObjectClassRowData::Evaluate(const UObject* LeftHandSide) const
{
	if (Value && LeftHandSide)
	{
		switch (Comparison)
		{
			case EObjectClassColumnCellValueComparison::Equal:
				return LeftHandSide->GetClass() == Value.Get();
				
			case EObjectClassColumnCellValueComparison::NotEqual:
				return LeftHandSide->GetClass() != Value.Get();
                				
			case EObjectClassColumnCellValueComparison::SubClassOf:
				return LeftHandSide->GetClass()->IsChildOf(Value.Get());
				
			case EObjectClassColumnCellValueComparison::NotSubClassOf:
				return !LeftHandSide->GetClass()->IsChildOf(Value.Get());

			default:
				return false;
		}
	}
	// always pass if the class was not set
	return true;
}

void FObjectClassColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	UObject* Result = nullptr;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterObjectBase>().GetValue(Context, Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result->GetPathName());	
#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif
		
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValues.IsValidIndex(IndexData.Index))
			{
				const FChooserObjectClassRowData& RowValue = RowValues[IndexData.Index];
				if (RowValue.Evaluate(Result))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}
#if WITH_EDITOR
	void FObjectClassColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterObjectBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserObjectClassRowData::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FObjectClassColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserObjectClassRowData::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FChooserObjectClassRowData>();
		}
	}
#endif
