// Copyright Epic Games, Inc. All Rights Reserved.
#include "ObjectColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectColumn)

bool FObjectContextProperty::GetValue(FChooserEvaluationContext& Context, FSoftObjectPath& OutResult) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			// if the property is a soft object property, get the path directly
			if (Result.PropertyType == UE::Chooser::EChooserPropertyAccessType::SoftObjectRef)
			{
				const FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<const FSoftObjectPtr*>(Result.Container + Result.PropertyOffset);
				OutResult = SoftObjectPtr.ToSoftObjectPath();
				return true;
			}
		
			// otherwise get the value from the object property and convert to a soft object path
			const UObject** LoadedObject = reinterpret_cast<const UObject**>(Result.Container + Result.PropertyOffset);
			OutResult = *LoadedObject;
			return true;
		}
	}

	return false;
}

bool FObjectContextProperty::GetValue(FChooserEvaluationContext& Context, UObject*& OutResult) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			// if the property is a soft object property, get the path directly
			if (Result.PropertyType == UE::Chooser::EChooserPropertyAccessType::SoftObjectRef)
			{
				const FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<const FSoftObjectPtr*>(Result.Container + Result.PropertyOffset);
				OutResult = SoftObjectPtr.Get();
				return true;
			}
		
			// otherwise get the value from the object property and convert to a soft object path
			UObject** LoadedObject = reinterpret_cast<UObject**>(Result.Container + Result.PropertyOffset);
			OutResult = *LoadedObject;
			return true;
		}
	}

	return false;
}


bool FObjectContextProperty::SetValue(FChooserEvaluationContext& Context, UObject* OutputObject) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			if (Result.PropertyType == UE::Chooser::EChooserPropertyAccessType::SoftObjectRef)
			{
				FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<FSoftObjectPtr*>(Result.Container + Result.PropertyOffset);
				SoftObjectPtr = OutputObject;
				return true;
			}
		
			UObject** ObjectPtr = reinterpret_cast<UObject**>(Result.Container + Result.PropertyOffset);
			*ObjectPtr = OutputObject;
			return true;
		}
	}

	return false;
}

bool FObjectContextProperty::SetValue(FChooserEvaluationContext& Context, const FSoftObjectPath& InValue) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (UE::Chooser::ResolvePropertyChain(Context, Binding, Result))
	{
		if (Result.Function == nullptr)
		{
			if (Result.PropertyType == UE::Chooser::EChooserPropertyAccessType::SoftObjectRef)
			{
				FSoftObjectPtr& SoftObjectPtr = *reinterpret_cast<FSoftObjectPtr*>(Result.Container + Result.PropertyOffset);
				SoftObjectPtr = InValue;
				return true;
			}
		
			UObject** ObjectPtr = reinterpret_cast<UObject**>(Result.Container + Result.PropertyOffset);
			*ObjectPtr = InValue.ResolveObject();
			return true;
		}
	}

	return false;
}

FObjectColumn::FObjectColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FObjectContextProperty::StaticStruct());
#endif
}

bool FChooserObjectRowData::Evaluate(const FSoftObjectPath& LeftHandSide) const
{
	switch (Comparison)
	{
		case EObjectColumnCellValueComparison::MatchEqual:
			return LeftHandSide == Value.ToSoftObjectPath();

		case EObjectColumnCellValueComparison::MatchNotEqual:
			return LeftHandSide != Value.ToSoftObjectPath();

		case EObjectColumnCellValueComparison::MatchAny:
			return true;

		default:
			return false;
	}
}

void FObjectColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	FSoftObjectPath Result;
	if (InputValue.IsValid() &&
		InputValue.Get<FChooserParameterObjectBase>().GetValue(Context, Result))
	{
		TRACE_CHOOSER_VALUE(Context, ToCStr(InputValue.Get<FChooserParameterBase>().GetDebugName()), Result.ToString());
	
#if WITH_EDITOR
		if (Context.DebuggingInfo.bCurrentDebugTarget)
		{
			TestValue = Result;
		}
#endif
		
		for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
		{
			if (RowValues.IsValidIndex(IndexData.Index))
			{
				const FChooserObjectRowData& RowValue = RowValues[IndexData.Index];
				if (RowValue.Evaluate(Result))
				{
					IndexListOut.Push(IndexData);
				}
			}
		}
	}
	else
	{
		// passthrough fallback (behaves better during live editing)
		IndexListOut = IndexListIn;
	}
}
#if WITH_EDITOR
	void FObjectColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterObjectBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserObjectRowData::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, RowValues[RowIndex]);
	}

	void FObjectColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserObjectRowData::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			RowValues[RowIndex] = StructView->Get<FChooserObjectRowData>();
		}
	}
#endif
