// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputBoolColumn.h"
#include "ChooserPropertyAccess.h"
#include "BoolColumn.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputBoolColumn)

FOutputBoolColumn::FOutputBoolColumn()
{
	InputValue.InitializeAs(FBoolContextProperty::StaticStruct());
}

void FOutputBoolColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			bool bValue = GetValueForIndex(RowIndex);
			InputValue.Get<FChooserParameterBoolBase>().SetValue(Context, bValue);
						
			#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = bValue;
			}
			#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputBoolColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputBoolColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FOutputBoolColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBoolBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Bool);
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueFloat(PropertyName, GetValueForIndex(RowIndex));
	}

	void FOutputBoolColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
			
		TValueOrError<bool, EPropertyBagResult> Result = PropertyBag.GetValueBool(PropertyName);
		if (bool* Value = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = *Value;
		}
	} 

#endif
