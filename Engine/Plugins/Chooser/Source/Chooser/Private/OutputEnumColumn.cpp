// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputEnumColumn.h"
#include "EnumColumn.h"
#include "ChooserPropertyAccess.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputEnumColumn)

FOutputEnumColumn::FOutputEnumColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FEnumContextProperty::StaticStruct());
#endif
}

void FOutputEnumColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			const FChooserOutputEnumRowData& Value = GetValueForIndex(RowIndex);
			InputValue.Get<FChooserParameterEnumBase>().SetValue(Context, Value.Value);
						
#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = Value.Value;
			}
#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputEnumColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputEnumColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITORONLY_DATA
void FOutputEnumColumn::PostLoad()
{
	Super::PostLoad();
    	
    if (InputValue.IsValid())
    {
    	InputValue.GetMutable<FChooserParameterBase>().PostLoad();
    	
	    if (const UEnum* Enum = InputValue.Get<FChooserParameterEnumBase>().GetEnum())
	    {
	    	for(FChooserOutputEnumRowData& CellData : RowValues)
	    	{
	    		if (Enum)
	    		{
	    			if (Enum->IsValidEnumName(CellData.ValueName))
	    			{
	    				CellData.Value = Enum->GetValueByName(CellData.ValueName);
	    			}
	    			else
					// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
					{
						CellData.ValueName = Enum->GetNameByValue(CellData.Value);
					}
	    		}
	    	}
	    	if (Enum->IsValidEnumName(FallbackValue.ValueName))
			{
				FallbackValue.Value = Enum->GetValueByName(FallbackValue.ValueName);
			}
			else
			// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
			{
				FallbackValue.ValueName = Enum->GetNameByValue(FallbackValue.Value);
			}
	    }
    }
}
#endif

#if WITH_EDITOR

	void FOutputEnumColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);

		// make another property bag in place of our struct, just so that the value enum will be correctly typed in the details panel
		
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Enum, InputValue.Get<FChooserParameterEnumBase>().GetEnum());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueEnum(PropertyName, GetValueForIndex(RowIndex).Value, InputValue.Get<FChooserParameterEnumBase>().GetEnum());
	}
	
	void FOutputEnumColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData",ColumnIndex);
		
		TValueOrError<uint8, EPropertyBagResult> Result = PropertyBag.GetValueEnum(PropertyName, InputValue.Get<FChooserParameterEnumBase>().GetEnum());
		if (uint8* Value = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex).Value = *Value;
		}
	}

void FOutputEnumColumn::EnumChanged(const UEnum* Enum)
{
	if (InputValue.IsValid())
	{
		if (Enum == InputValue.Get<FChooserParameterEnumBase>().GetEnum())
		{
			for(FChooserOutputEnumRowData& CellData : RowValues)
			{
				if (Enum->IsValidEnumName(CellData.ValueName))
				{
					CellData.Value = Enum->GetValueByName(CellData.ValueName);
				}
				else
				// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
				{
					CellData.ValueName = Enum->GetNameByValue(CellData.Value);
				}
			}
			
			if (Enum->IsValidEnumName(FallbackValue.ValueName))
			{
				FallbackValue.Value = Enum->GetValueByName(FallbackValue.ValueName);
			}
			else
			// if StringValue is empty (or the names in the enum have changed) upgrade old data, to have a valid Name
			{
				FallbackValue.ValueName = Enum->GetNameByValue(FallbackValue.Value);
			}
		}
	}
}
#endif
