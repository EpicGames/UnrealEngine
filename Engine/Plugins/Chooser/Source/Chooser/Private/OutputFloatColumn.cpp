// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputFloatColumn.h"
#include "ChooserPropertyAccess.h"
#include "FloatRangeColumn.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputFloatColumn)

FOutputFloatColumn::FOutputFloatColumn()
{
	InputValue.InitializeAs(FFloatContextProperty::StaticStruct());
}

void FOutputFloatColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			double Value = GetValueForIndex(RowIndex);
			InputValue.Get<FChooserParameterFloatBase>().SetValue(Context, Value);
						
			#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = Value;
			}
			#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputFloatColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputFloatColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FOutputFloatColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Float);
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueFloat(PropertyName, GetValueForIndex(RowIndex));
	}

	void FOutputFloatColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
			
		TValueOrError<float, EPropertyBagResult> Result = PropertyBag.GetValueFloat(PropertyName);
		if (float* Value = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = *Value;
		}
	}
#endif
