// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputGameplayTagColumn.h"

#include "ChooserIndexArray.h"
#include "ChooserTrace.h"
#include "GameplayTagColumn.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputGameplayTagColumn)

FOutputGameplayTagColumn::FOutputGameplayTagColumn()
{
	InputValue.InitializeAs(FGameplayTagContextProperty::StaticStruct());
}

void FOutputGameplayTagColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			const FGameplayTagContainer& OutputTags = GetValueForIndex(RowIndex);
			InputValue.Get<FChooserParameterGameplayTagBase>().SetValue(Context, OutputTags);
#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = OutputTags;
			}
#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputGameplayTagColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputGameplayTagColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FOutputGameplayTagColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData", ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Struct, FGameplayTagContainer::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, GetValueForIndex(RowIndex));
	}

	void FOutputGameplayTagColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);

		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FGameplayTagContainer::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = StructView->Get<FGameplayTagContainer>();
		}
	}
#endif
