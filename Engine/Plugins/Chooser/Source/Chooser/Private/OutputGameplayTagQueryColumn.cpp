// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputGameplayTagQueryColumn.h"

#include "ChooserIndexArray.h"
#include "ChooserTrace.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputGameplayTagQueryColumn)


bool FGameplayTagQueryContextProperty::SetValue(FChooserEvaluationContext& Context, const FGameplayTagQuery& Value) const
{
	return Binding.SetValue(Context, Value);
}

FOutputGameplayTagQueryColumn::FOutputGameplayTagQueryColumn()
{
	InputValue.InitializeAs(FGameplayTagQueryContextProperty::StaticStruct());
}

void FOutputGameplayTagQueryColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			const FGameplayTagQuery& OutputTagQuery = GetValueForIndex(RowIndex);
			InputValue.Get<FChooserParameterGameplayTagQueryBase>().SetValue(Context, OutputTagQuery);
#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = OutputTagQuery;
			}
#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputGameplayTagQueryColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputGameplayTagQueryColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FOutputGameplayTagQueryColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData", ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Struct, FGameplayTagQuery::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueStruct(PropertyName, GetValueForIndex(RowIndex));
	}

	void FOutputGameplayTagQueryColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);

		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FGameplayTagQuery::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = StructView->Get<FGameplayTagQuery>();
		}
	}
#endif
