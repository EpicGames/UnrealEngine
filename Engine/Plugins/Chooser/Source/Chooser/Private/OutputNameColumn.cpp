// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputNameColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"
#include "NameColumn.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputNameColumn)

FChooserOutputNameColumn::FChooserOutputNameColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FNameContextProperty::StaticStruct());
#endif
}

void FChooserOutputNameColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			FName Value = GetValueForIndex(RowIndex);
			InputValue.Get<FNameContextProperty>().SetValue(Context, Value);
						
			#if WITH_EDITOR
			if (Context.DebuggingInfo.bCurrentDebugTarget)
			{
				TestValue = Value;
			}
			#endif
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FChooserOutputNameColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FChooserOutputNameColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FChooserOutputNameColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterNameBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Name);
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
		PropertyBag.SetValueName(PropertyName, GetValueForIndex(RowIndex));
	}

	void FChooserOutputNameColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);

		TValueOrError<FName, EPropertyBagResult> Result = PropertyBag.GetValueName(PropertyName);
		if (FName* Name = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = *Name;
		}
	}
#endif
