// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputObjectColumn.h"
#include "ObjectColumn.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTrace.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputObjectColumn)

FOutputObjectColumn::FOutputObjectColumn()
{
#if WITH_EDITOR
	InputValue.InitializeAs(FObjectContextProperty::StaticStruct());
#endif
}

void FOutputObjectColumn::Compile(IHasContextClass* Owner, bool bForce)
{
	FChooserColumnBase::Compile(Owner, bForce);

	for(FChooserOutputObjectRowData& Value : RowValues)
	{
		if (Value.Value.IsValid())
		{
			FObjectChooserBase& Result = Value.Value.GetMutable<FObjectChooserBase>();
			Result.Compile(Owner, bForce);
		}
	}
}

void FOutputObjectColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			TSoftObjectPtr<UObject> Result;
			const FObjectChooserBase* Value = GetValueForIndex(RowIndex).Value.GetPtr<FObjectChooserBase>();
			if (Value)
			{
				Value->ChooseObject(Context, Result);
			}
			if (const FChooserParameterObjectBase* OutputParameter = InputValue.GetPtr<FChooserParameterObjectBase>())
			{
				OutputParameter->SetValue(Context, Result.ToSoftObjectPath());
			}
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputObjectColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputObjectColumn::SetOutputs", RowIndex);
#endif
		}
	}
}

#if WITH_EDITOR
	void FOutputObjectColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FText DisplayName;
		InputValue.Get<FChooserParameterObjectBase>().GetDisplayName(DisplayName);
		FName PropertyName("RowData",ColumnIndex);
		FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, FChooserOutputObjectRowData::StaticStruct());
		PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
		PropertyBag.AddProperties({PropertyDesc});
	
		PropertyBag.SetValueStruct(PropertyName, GetValueForIndex(RowIndex));
	}

	void FOutputObjectColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, FChooserOutputObjectRowData::StaticStruct());
		if (FStructView* StructView = Result.TryGetValue())
		{
			GetValueForIndex(RowIndex) = StructView->Get<FChooserOutputObjectRowData>();
		}
	}

#endif
