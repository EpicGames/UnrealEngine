// Copyright Epic Games, Inc. All Rights Reserved.
#include "OutputStructColumn.h"
#include "ChooserPropertyAccess.h"

#if WITH_EDITOR
#include "IPropertyAccessEditor.h"
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutputStructColumn)

bool FStructContextProperty::SetValue(FChooserEvaluationContext& Context, const FInstancedStruct& InValue) const
{
	void* TargetData;
	const UStruct* StructType;
	if (Binding.GetStructPtr(Context, TargetData, StructType))
	{
		if (StructType == InValue.GetScriptStruct())
		{
			InValue.GetScriptStruct()->CopyScriptStruct(TargetData, InValue.GetMemory());
			return true;
		}
		else
		{
			return false;
		}
	}
	return false;
}

#if WITH_EDITOR

void FOutputStructColumn::InputValueChanged()
{
	StructTypeChanged();
}

void FOutputStructColumn::StructTypeChanged()
{
	if (InputValue.IsValid())
	{
		const UScriptStruct* Struct = InputValue.Get<FChooserParameterStructBase>().GetStructType();

		if (DefaultRowValue.GetScriptStruct() != Struct)
		{
			DefaultRowValue.InitializeAs(Struct);
		}

		if (FallbackValue.GetScriptStruct() != Struct)
		{
			FallbackValue.InitializeAs(Struct);
		}

		for (FInstancedStruct& RowValue : RowValues)
		{
			if (RowValue.GetScriptStruct() != Struct)
			{
				RowValue.InitializeAs(Struct);
			}
		}
	}
}

void FOutputStructColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	if (InputValue.IsValid())
	{
		const FChooserParameterStructBase& StructInput = InputValue.Get<FChooserParameterStructBase>();

		// if there is no struct bound, the row values have no type, and can't be added to the details panel, so just skip them
		if (StructInput.GetStructType())
		{
			FText DisplayName;
			StructInput.GetDisplayName(DisplayName);
			FName PropertyName("RowData",ColumnIndex);
			
			FInstancedStruct& Value = GetValueForIndex(RowIndex);
			FPropertyBagPropertyDesc PropertyDesc(PropertyName,  EPropertyBagPropertyType::Struct, Value.GetScriptStruct());
			PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", DisplayName.ToString()));
			PropertyBag.AddProperties({PropertyDesc});
			PropertyBag.SetValueStruct(PropertyName, FConstStructView(Value.GetScriptStruct(), Value.GetMemory()));
		}
	}
}

void FOutputStructColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
{
	if (InputValue.IsValid())
	{
		const FChooserParameterStructBase& StructInput = InputValue.Get<FChooserParameterStructBase>();
		if (StructInput.GetStructType())
		{
			FName PropertyName("RowData", ColumnIndex);

			FInstancedStruct& Value = GetValueForIndex(RowIndex);

			TValueOrError<FStructView, EPropertyBagResult> Result = PropertyBag.GetValueStruct(PropertyName, Value.GetScriptStruct());
			if (FStructView* StructView = Result.TryGetValue())
			{
				GetValueForIndex(RowIndex).GetScriptStruct()->CopyScriptStruct(Value.GetMutableMemory(), StructView->GetMemory());
			}
		}
	}
}

#endif // WITH_EDITOR

FOutputStructColumn::FOutputStructColumn()
{
	InputValue.InitializeAs(FStructContextProperty::StaticStruct());
}

void FOutputStructColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid())
	{
		if (RowIndex == ChooserColumn_SpecialIndex_Fallback || RowValues.IsValidIndex(RowIndex))
		{
			const FInstancedStruct& OutputValue = GetValueForIndex(RowIndex);
			
			if (OutputValue.IsValid())
			{
				InputValue.Get<FChooserParameterStructBase>().SetValue(Context, OutputValue);
#if WITH_EDITOR
				if (Context.DebuggingInfo.bCurrentDebugTarget)
				{
					TestValue = OutputValue;
				}
#endif
			}
		}
		else
		{
#if CHOOSER_DEBUGGING_ENABLED
			UE_ASSET_LOG(LogChooser, Error, Context.DebuggingInfo.CurrentChooser, TEXT("Invalid index %d passed to FOutputStructColumn::SetOutputs"), RowIndex);
#else
			UE_LOGF(LogChooser, Error, "Invalid index %d passed to FOutputStructColumn::SetOutputs", RowIndex);
#endif
		}
	}
}
