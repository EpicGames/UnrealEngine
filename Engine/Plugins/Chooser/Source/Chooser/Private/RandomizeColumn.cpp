// Copyright Epic Games, Inc. All Rights Reserved.
#include "RandomizeColumn.h"
#include "ChooserIndexArray.h"
#include "ChooserPropertyAccess.h"
#if WITH_EDITOR
#include "StructUtils/PropertyBag.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RandomizeColumn)

bool FRandomizeContextProperty::GetValue(FChooserEvaluationContext& Context, FChooserRandomizationContext*& OutResult) const
{
	return Binding.GetValuePtr(Context,OutResult);
}

FRandomizeColumn::FRandomizeColumn()
{
	InputValue.InitializeAs(FRandomizeContextProperty::StaticStruct());
}

float GetRepeatProbabilityMultiplier(const TArray<float>& RepeatProbabilityMultipliers, const FRandomizationState* State, uint32 Index)
{
	if (State)
	{
		int32 RecentlySelectedIndex =  State->RecentlySelectedRows.Find(Index);
		if (RecentlySelectedIndex != INDEX_NONE)
		{
			// RecentlySelectedRows array is a ring buffer.  Remap to where 0 means it was selected	most recently
			RecentlySelectedIndex = (State->LastSelectedRow  - RecentlySelectedIndex + State->RecentlySelectedRows.Num())  % State->RecentlySelectedRows.Num();
			return RepeatProbabilityMultipliers[RecentlySelectedIndex];
		}
	}
	return 1.0;
}

void FRandomizeColumn::Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const
{
	int Count = IndexListIn.Num();
	int Selection = 0;

	FChooserRandomizationContext* RandomizationContext = nullptr;
	if (InputValue.IsValid())
	{
		InputValue.Get<FChooserParameterRandomizeBase>().GetValue(Context,RandomizationContext);
	}
	
	if (Count > 1)
	{
		FRandomizationState* State  = nullptr;
		if (RandomizationContext)
		{
			State = &RandomizationContext->StateMap.FindOrAdd(this, FRandomizationState());

			if (State->RecentlySelectedRows.Num() != RepeatProbabilityMultipliers.Num())
			{
				State->RecentlySelectedRows.Init(-1, RepeatProbabilityMultipliers.Num());
				State->LastSelectedRow = 0;
			}
		}

		if (IndexListIn.HasCosts())
		{
			// find the lowest cost row
			float LowestCost = UE_MAX_FLT;
			uint32 LowestCostIndex = 0;
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (IndexData.Cost < LowestCost)
				{
					LowestCost = IndexData.Cost;
					LowestCostIndex = IndexData.Index;
				}
			}

			// compute the sum of all weights/probabilities - only considering rows with cost nearly equal to the lowest cost
			float TotalWeight = 0;
			int MinCount = 0;
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				if (FMath::IsNearlyEqual(LowestCost, IndexData.Cost, EqualCostThreshold))
				{
					float RowWeight = 1.0f;
					if (RowValues.Num() > static_cast<int32>(IndexData.Index))
					{
						RowWeight = RowValues[IndexData.Index];
					}

					RowWeight = RowWeight * GetRepeatProbabilityMultiplier(RepeatProbabilityMultipliers, State, IndexData.Index);
			
					TotalWeight += RowWeight;
					MinCount++;
				}
			}

			if (MinCount == 1)
			{
				// only one entry with lowest cost, we can't randomize
				IndexListOut.Push({LowestCostIndex, LowestCost});
				return;
			}

			float AddWeight = 0.0f;
			if (TotalWeight < UE_SMALL_NUMBER)
			{
				// when total weight is 0, and there is more than one entry, fake uniform weights;
				TotalWeight = 1.0f;
				AddWeight = 1.0f / static_cast<float>(MinCount);
			}

			// pick a random float from 0-total weight
			const float RandomNumber = Context.RandomStream ? Context.RandomStream->RandRange(0.f, TotalWeight) : FMath::FRandRange(0.0f, TotalWeight);
			float Weight = 0;

			// add up the weights again, and select the index where our sum clears the random float
			for (; Selection < Count - 1; Selection++)
			{
				const FChooserIndexArray::FIndexData& IndexData = IndexListIn[Selection];
				if (FMath::IsNearlyEqual(LowestCost, IndexData.Cost, EqualCostThreshold))
				{
					float RowWeight = 1.0f;
				
					if (RowValues.IsValidIndex(IndexData.Index))
					{
						RowWeight = RowValues[IndexData.Index];
					}

					RowWeight = AddWeight + RowWeight * GetRepeatProbabilityMultiplier(RepeatProbabilityMultipliers, State, IndexData.Index);
				
					Weight += RowWeight;
					if (Weight > RandomNumber)
					{
						break;
					}
				}
			}	
		}
		else
		{
			// compute the sum of all weights/probabilities
			float TotalWeight = 0;
			for (const FChooserIndexArray::FIndexData& IndexData : IndexListIn)
			{
				float RowWeight = 1.0f;
				if (RowValues.Num() > static_cast<int32>(IndexData.Index))
				{
					RowWeight = RowValues[IndexData.Index];
				}

				RowWeight = RowWeight * GetRepeatProbabilityMultiplier(RepeatProbabilityMultipliers, State, IndexData.Index);
			
				TotalWeight += RowWeight;
			}

			float AddWeight = 0.0f;
			if (TotalWeight < UE_SMALL_NUMBER)
			{
				// when total weight is 0, and there is more than one entry, fake uniform weights;
				TotalWeight = 1.0f;
				AddWeight = 1.0f / static_cast<float>(IndexListIn.Num());
			}

			// pick a random float from 0-total weight
			const float RandomNumber = Context.RandomStream ? Context.RandomStream->RandRange(0.f, TotalWeight) : FMath::FRandRange(0.0f, TotalWeight);
			float Weight = 0;

			// add up the weights again, and select the index where our sum clears the random float
			for (; Selection < Count - 1; Selection++)
			{
				const FChooserIndexArray::FIndexData& IndexData = IndexListIn[Selection];
				float RowWeight = 1.0f;
			
				if (RowValues.IsValidIndex(IndexData.Index))
				{
					RowWeight = RowValues[IndexData.Index];
				}
			
				RowWeight = AddWeight + RowWeight * GetRepeatProbabilityMultiplier(RepeatProbabilityMultipliers, State, IndexData.Index);
			
				Weight += RowWeight;
				if (Weight > RandomNumber)
				{
					break;
				}
			}
		}
	}

	if (Selection < Count)
	{
		IndexListOut.Push(IndexListIn[Selection]);
	}
}

void FRandomizeColumn::SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const
{
	if (InputValue.IsValid() && RowValues.IsValidIndex((RowIndex)))
	{
		FChooserRandomizationContext* RandomizationContext = nullptr;
		InputValue.Get<FChooserParameterRandomizeBase>().GetValue(Context,RandomizationContext);
		if(RandomizationContext)
		{
			if (FRandomizationState* State = RandomizationContext->StateMap.Find(this))
			{
				if (State->RecentlySelectedRows.Num() > 0)
				{
					State->LastSelectedRow = (State->LastSelectedRow + 1) % State->RecentlySelectedRows.Num();
					State->RecentlySelectedRows[State->LastSelectedRow] = RowIndex;
				}
			}
		}
	}
}
	#if WITH_EDITOR

	void FRandomizeColumn::AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		if (RowValues.IsValidIndex(RowIndex))
		{
			FName PropertyName("RowData",ColumnIndex);
			FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Float);
			PropertyDesc.MetaData.Add(FPropertyBagPropertyDescMetaData("DisplayName", "Randomize"));
			PropertyBag.AddProperties({PropertyDesc});
			PropertyBag.SetValueFloat(PropertyName, RowValues[RowIndex]);
		}
	}

	void FRandomizeColumn::SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex)
	{
		FName PropertyName("RowData", ColumnIndex);
		
		TValueOrError<float, EPropertyBagResult> Result = PropertyBag.GetValueFloat(PropertyName);
		if (float* Value = Result.TryGetValue())
		{
			RowValues[RowIndex] = *Value;
		}
	}

    #endif
