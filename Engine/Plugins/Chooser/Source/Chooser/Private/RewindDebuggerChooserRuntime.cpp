// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerChooserRuntime.h"

void FRewindDebuggerChooserRuntime::RecordingStarted()
{
	UE::Trace::ToggleChannel(TEXT("Chooser"), true);
}

void FRewindDebuggerChooserRuntime::RecordingStopped()
{
	UE::Trace::ToggleChannel(TEXT("Chooser"), false);
}