// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

// Rewind debugger extension for Chooser support

class FRewindDebuggerChooserRuntime : public IRewindDebuggerRuntimeExtension
{
public:
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
};
