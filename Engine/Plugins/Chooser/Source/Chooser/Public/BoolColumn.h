// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterBool.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "BoolColumn.generated.h"

#define UE_API CHOOSER_API

UENUM()
enum class EBoolColumnCellValue : uint8
{
	MatchFalse = 0,
	MatchTrue = 1,
	MatchAny = 2,
};


USTRUCT(DisplayName = "Bool Property Binding")
struct FBoolContextProperty :  public FChooserParameterBoolBase
{
	GENERATED_BODY()
public:

	UPROPERTY()
	TArray<FName> PropertyBindingChain_DEPRECATED;
	
	UPROPERTY(EditAnywhere, Meta = (BindingType = "bool", BindingAllowFunctions = "true", BindingColor = "BooleanPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;
	
	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, bool& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, bool InValue) const override;

	virtual void PostLoad() override
	{
		if (PropertyBindingChain_DEPRECATED.Num() > 0)
		{
			Binding.PropertyBindingChain = PropertyBindingChain_DEPRECATED;
			PropertyBindingChain_DEPRECATED.SetNum(0);
		}
	}
	
	CHOOSER_PARAMETER_BOILERPLATE();
	
};

USTRUCT(DisplayName = "Bool", Meta = (Category = "Filter", Tooltip = "A column that filters rows based on the value of a Bool."))
struct FBoolColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FBoolColumn();
	
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterBoolBase", ToolTip="The bool property this column will filter based on" ), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<bool> RowValues_DEPRECATED;

	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	EBoolColumnCellValue DefaultRowValue = EBoolColumnCellValue::MatchAny;
#endif
	
	UPROPERTY(EditAnywhere, Category= "Data", DisplayName="RowValues");
	TArray<EBoolColumnCellValue> RowValuesWithAny; 
	
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable bool TestValue = false;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return RowValuesWithAny.IsValidIndex(RowIndex) && (RowValuesWithAny[RowIndex] == EBoolColumnCellValue::MatchAny || TestValue == static_cast<bool>(RowValuesWithAny[RowIndex]));
	}
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		Reader << TestValue;
	}
	
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	virtual void PostLoad() override
	{
#if WITH_EDITORONLY_DATA
		if (RowValues_DEPRECATED.Num() > 0)
		{
			RowValuesWithAny.SetNum(0,EAllowShrinking::No);
			RowValuesWithAny.Reserve(RowValues_DEPRECATED.Num());
			for(bool Value : RowValues_DEPRECATED)
			{
				RowValuesWithAny.Add(Value ? EBoolColumnCellValue::MatchTrue : EBoolColumnCellValue::MatchFalse);
			}
			RowValues_DEPRECATED.SetNum(0);
		}
#endif
		
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	CHOOSER_COLUMN_BOILERPLATE2(FChooserParameterBoolBase, RowValuesWithAny);
};

// deprecated class versions for converting old data

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserParameterBool_ContextProperty :  public UObject, public IChooserParameterBool
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FName> PropertyBindingChain;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FBoolContextProperty::StaticStruct());
		FBoolContextProperty& Property = OutInstancedStruct.GetMutable<FBoolContextProperty>();
		Property.Binding.PropertyBindingChain = PropertyBindingChain;
	}
};

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserColumnBool : public UObject, public IChooserColumn
{
	GENERATED_BODY()
public:
	UDEPRECATED_ChooserColumnBool() {};
	UDEPRECATED_ChooserColumnBool(const FObjectInitializer& ObjectInitializer)
	{
		InputValue = ObjectInitializer.CreateDefaultSubobject<UDEPRECATED_ChooserParameterBool_ContextProperty>(this, "InputValue");
	}	
	
	UPROPERTY(EditAnywhere, Category = "Input")
	TScriptInterface<IChooserParameterBool> InputValue;

	UPROPERTY(EditAnywhere, Category=Runtime)
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<bool> RowValues;

	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FBoolColumn::StaticStruct());
		FBoolColumn& Column = OutInstancedStruct.GetMutable<FBoolColumn>();
		if (IChooserParameterBool* InputValueInterface = InputValue.GetInterface())
		{
			InputValueInterface->ConvertToInstancedStruct(Column.InputValue);
		}

		Column.RowValuesWithAny.SetNum(0,EAllowShrinking::No);
		Column.RowValuesWithAny.Reserve(RowValues.Num());
		for(bool Value : RowValues)
		{
			Column.RowValuesWithAny.Add(Value ? EBoolColumnCellValue::MatchTrue : EBoolColumnCellValue::MatchFalse);
		}
	}
};

#undef UE_API
