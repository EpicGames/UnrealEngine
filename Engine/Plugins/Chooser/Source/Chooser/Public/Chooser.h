// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IObjectChooser.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserColumn.h"
#include "IHasContext.h"
#include "ChooserSignature.h"
#if WITH_EDITOR
#include "Kismet2/StructureEditorUtils.h"
#endif

#include "Chooser.generated.h"

/**
* Data table used to choose an asset based on input parameters
*/
UCLASS(MinimalAPI, BlueprintType)
class UChooserTable : public UChooserSignature
{
	GENERATED_UCLASS_BODY()
public:
	UChooserTable() {}
	CHOOSER_API virtual void BeginDestroy() override;

	CHOOSER_API virtual void PostLoad() override;
	CHOOSER_API virtual void Compile(bool bForce = false) override;
	CHOOSER_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITORONLY_DATA
	CHOOSER_API void RemoveDisabledData();
	CHOOSER_API void PopulateCookedData(bool bCooking);
	CHOOSER_API void DeleteNestedChooser(UChooserTable* NestedChooser);
	CHOOSER_API void DeleteNestedObject(UObject* ObjectToDelete);
#endif

	bool IsRowDisabled(int32 RowIndex) const
	{
#if WITH_EDITORONLY_DATA
		return !IsCookedData() && DisabledRows.IsValidIndex(RowIndex) && DisabledRows[RowIndex];
#else
		return false;
#endif
	}
#if WITH_EDITOR
	CHOOSER_API	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	CHOOSER_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	CHOOSER_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	void OnDependentStructChanged(UUserDefinedStruct* Blueprint) { Compile(true); }
	void OnDependencyCompiled(UBlueprint* Blueprint) { Compile(true); }
	CHOOSER_API virtual void AddCompileDependency(const UStruct* Struct) override;
	FChooserOutputObjectTypeChanged OnOutputObjectTypeChanged;
	
	CHOOSER_API virtual void PostEditUndo() override;
	CHOOSER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	
	void SetDebugSelectedRow(int32 Index) const { DebugSelectedRows = { Index }; }
	void SetDebugSelectedRows(const TArray<int32>& Indices) const { DebugSelectedRows = Indices; }
	const TArray<int32>& GetDebugSelectedRows() const { return DebugSelectedRows; }
	bool HasDebugTarget() const { return !DebugTargetName.IsEmpty(); }
	
	void SetDebugTarget(FString Name) { DebugTargetName = Name; }
	void ResetDebugTarget() { DebugTargetName = ""; }
	const FString& GetDebugTargetName() const { return DebugTargetName; }
	

	CHOOSER_API void AddRecentContextObject(const FString& ObjectName) const;
	CHOOSER_API void IterateRecentContextObjects(TFunction<void(const FString&)> Callback) const;
	CHOOSER_API void UpdateDebugging(FChooserEvaluationContext& Context) const;
	
	static CHOOSER_API const FName PropertyNamesTag;
	static CHOOSER_API const FString PropertyTagDelimiter;

		
	void SetEnableDebugTesting(bool bValue)
	{
		GetRootChooser()->bEnableDebugTesting = bValue;
	}
	
	bool GetEnableDebugTesting() const
	{
		return GetRootChooser()->bEnableDebugTesting;
	}
	
	void SetDebugTestValuesValid(bool bValue) const
   	{
   		GetRootChooser()->bDebugTestValuesValid = bValue;
   	}
	
	bool GetDebugTestValuesValid() const
	{
		return GetRootChooser()->bDebugTestValuesValid;
	}

private:
	// enable display of which cells pass/fail based on current TestValue for each column
	bool bEnableDebugTesting = false;
	mutable bool bDebugTestValuesValid = false;
	
	// caching the OutputObjectType and ContextObjectType so that on Undo, we can tell if we should fire the changed delegate
	UClass* CachedPreviousOutputObjectType = nullptr;
	EObjectChooserResultType CachedPreviousResultType = EObjectChooserResultType::ObjectResult;
	
	// objects this chooser has been recently evaluated on
	mutable TArray<FString> RecentContextObjects;
	mutable FCriticalSection DebugLock;
	// reference to the UObject in PIE  which we want to get debug info for
	mutable TWeakObjectPtr<const UObject> DebugTarget;
	FString DebugTargetName;
	
	// Rows which were selected last time this chooser was evaluated on DebugTarget
	mutable TArray<int32> DebugSelectedRows;

	TArray<TWeakObjectPtr<UStruct>> CompileDependencies;
#endif

public:
#if WITH_EDITORONLY_DATA
	// deprecated UObject Results
	UPROPERTY()
	TArray<TScriptInterface<IObjectChooser>> Results_DEPRECATED;

	// deprecated single context object
	UPROPERTY()
	TObjectPtr<UClass> ContextObjectType_DEPRECATED;
	
	// deprecated UObject Columns
	UPROPERTY()
	TArray<TScriptInterface<IChooserColumn>> Columns_DEPRECATED;
#endif
	
	
	UChooserTable* GetRootChooser() { return RootChooser ? RootChooser.Get() : this; }
	const UChooserTable* GetRootChooser() const { return RootChooser ? RootChooser.Get() : this; }

	UChooserTable* GetContextOwner() { return RootChooser ? RootChooser.Get() : this; }
	const UChooserTable* GetContextOwner() const { return RootChooser ? RootChooser.Get() : this; }

	UPROPERTY()
	TObjectPtr<UChooserTable> RootChooser;
	
	// FallbackResult will be used as the Result if there are no rows in the chooser which pass all filters.  If FallbackResult is not assigned, then the Chooser will return null in that case.
	UPROPERTY()
	FInstancedStruct FallbackResult;
	
	// IHasContextClass implementation
	virtual TConstArrayView<FInstancedStruct> GetContextData() const override { return GetRootChooser()->ContextData; }
	
#if WITH_EDITORONLY_DATA
	// Each possible Result (Rows of chooser table)
	UPROPERTY()
	TArray<FInstancedStruct> ResultsStructs;

	UPROPERTY()
	TArray<bool> DisabledRows;

	UPROPERTY()
	TArray<TObjectPtr<UChooserTable>> NestedChoosers;
	
	UPROPERTY()
	TArray<TObjectPtr<UObject>> NestedObjects;

	CHOOSER_API void AddNestedChooser(UChooserTable* Chooser);
	CHOOSER_API void RemoveNestedChooser(UChooserTable* Chooser);
	FSimpleMulticastDelegate NestedChoosersChanged;
	
	CHOOSER_API void AddNestedObject(UObject* Object);
	CHOOSER_API void RemoveNestedObject(UObject* Object);
	FSimpleMulticastDelegate NestedObjectsChanged;

	// deprecated in favor of RootChooser
	UPROPERTY()
	TObjectPtr<UChooserTable> ParentTable;
	
	UPROPERTY()
	float EditorResultsColumnWidth = 300;

	UPROPERTY()
	uint32 Version = 0;

	static constexpr int CurrentVersion = 3;

private:
	bool bIsCookedPackage = false;

public:
#endif

	UFUNCTION()
	CHOOSER_API bool ResultAssetFilter(const FAssetData& AssetData);
	
	UPROPERTY()
	TArray<FInstancedStruct> CookedResults;

	// Columns which filter Results
	UPROPERTY(EditAnywhere, NoClear, DisplayName = "Columns", Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserColumnBase"), Category = "Hidden")
	TArray<FInstancedStruct> ColumnsStructs;
	
	bool IsCookedData() const
	{
#if WITH_EDITORONLY_DATA
		return bIsCookedPackage;
#else
		return true;
#endif
	}
	static CHOOSER_API FObjectChooserBase::EIteratorStatus EvaluateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback);
	static CHOOSER_API FObjectChooserBase::EIteratorStatus EvaluateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback Callback);
	static CHOOSER_API FObjectChooserBase::EIteratorStatus IterateChooser(FChooserEvaluationContext& Context, const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback);
	static CHOOSER_API FObjectChooserBase::EIteratorStatus IterateChooser(const UChooserTable* Chooser, FObjectChooserBase::FObjectChooserIteratorCallback Callback);
};

USTRUCT(DisplayName = "Nested Chooser", Meta = (Category = "Chooser", Tooltip = "Reference another ChooserTable embedded in this asset, which will be evaluated at runtime if this row is selected."))
struct FNestedChooser : public FObjectChooserBase
{
	GENERATED_BODY()

	CHOOSER_API virtual void ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const final override;
	CHOOSER_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	CHOOSER_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserIteratorCallback Callback) const final override;
	CHOOSER_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserSoftObjectIteratorCallback Callback) const final override;
	CHOOSER_API virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const final override;
	CHOOSER_API virtual void GetDebugName(FString& OutDebugName) const override;

	public:

	CHOOSER_API FNestedChooser();
	
	UPROPERTY()
	TObjectPtr<UChooserTable> Chooser;
};

USTRUCT(DisplayName = "Evaluate Chooser", Meta = (Category = "Chooser", Tooltip = "Reference another ChooserTable asset, which will be evaluated at runtime if this row is selected."))
struct FEvaluateChooser : public FObjectChooserBase
{
	GENERATED_BODY()
	CHOOSER_API virtual void ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const final override;
	CHOOSER_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	CHOOSER_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserIteratorCallback Callback) const final override;
	CHOOSER_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserSoftObjectIteratorCallback Callback) const final override;
	CHOOSER_API virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const final override;
	CHOOSER_API virtual void GetDebugName(FString& OutDebugName) const override;

	#if WITH_EDITOR
		virtual UObject* GetReferencedObject() const override { return Chooser; }
	#endif
	public:

	FEvaluateChooser() : Chooser(nullptr) {}
	FEvaluateChooser(TObjectPtr<UChooserTable> Table) : Chooser(Table) {}
	
	UPROPERTY(EditAnywhere, Category="Parameters")
	TObjectPtr<UChooserTable> Chooser;
};

// Deprecated class for converting old data
UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ObjectChooser_EvaluateChooser : public UObject, public IObjectChooser
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category="Parameters")
	TObjectPtr<UChooserTable> Chooser;

	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const
	{
		OutInstancedStruct.InitializeAs(FEvaluateChooser::StaticStruct());
		FEvaluateChooser& AssetChooser = OutInstancedStruct.GetMutable<FEvaluateChooser>();
		AssetChooser.Chooser = Chooser;
	}
};


UCLASS(MinimalAPI)
class UChooserColumnMenuContext : public UObject
{
	GENERATED_BODY()
public:
	class FAssetEditorToolkit* Editor;
	TWeakObjectPtr<UChooserTable> Chooser;
	int ColumnIndex;
};


