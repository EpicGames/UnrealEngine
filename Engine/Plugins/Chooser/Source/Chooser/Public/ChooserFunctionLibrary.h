// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Chooser.h"
#include "Templates/SubclassOf.h"
#include "UObject/SoftObjectPtr.h"
#include "ChooserFunctionLibrary.generated.h"

#define UE_API CHOOSER_API

/**
 * Chooser Function Library
 */
UCLASS(MinimalAPI)
class UChooserFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	/**
	* Evaluate a chooser table and return the selected UObject, or null
	*
	* @param ContextObject			(in) An Object from which the parameters to the Chooser Table will be read
	* @param ChooserTable			(in) The ChooserTable asset
	* @param ObjectClass			(in) Expected type of result object
	*/
	UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UE_API UObject* EvaluateChooser(const UObject* ContextObject, const UChooserTable* ChooserTable, TSubclassOf<UObject> ObjectClass);
	
	/**
    * Evaluate a chooser table and return the list of all selected UObjects
    *
    * @param ContextObject			(in) An Object from which the parameters to the Chooser Table will be read
    * @param ChooserTable			(in) The ChooserTable asset
    * @param ObjectClass			(in) Expected type of result objects
    */
    UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
    static UE_API TArray<UObject*> EvaluateChooserMulti(const UObject* ContextObject, const UChooserTable* ChooserTable, TSubclassOf<UObject> ObjectClass);
	
	/**
	* Evaluate an ObjectChooserBase and return the selected UObject, or null
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param ObjectChooser		(in) An Instanced struct containing an ObjectChooserBase implementation, such as EvaluateChooser, or EvaluateProxyAsset
	* @param ObjectClass		(in) Expected type of result object (or the type of UClass if bResultIsClass is true)
	* @param bResultIsClass		(in) The Object being returned is a UClass, and the ObjectClass parameter indicates what it must be a subclass of
	*/
	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UE_API UObject* EvaluateObjectChooserBase(UPARAM(Ref) struct FChooserEvaluationContext& Context,UPARAM(Ref) const struct FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass = false);
	
	
	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UE_API TSoftObjectPtr<UObject> EvaluateObjectChooserBaseSoft(UPARAM(Ref) struct FChooserEvaluationContext& Context,UPARAM(Ref) const struct FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass = false);

	/**
	* Evaluate a chooser table and return all selected UObjects
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param ObjectChooser		(in) An Instanced struct containing an ObjectChooserBase implementation, such as EvaluateChooser, or EvaluateProxyAsset
	* @param ObjectClass		(in) Expected type of result object (or the type of UClass if bResultIsClass is true)
	* @param bResultIsClass		(in) The Object being returned is a UClass, and the ObjectClass parameter indicates what it must be a subclass of
	*/
	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UE_API TArray<UObject*> EvaluateObjectChooserBaseMulti(UPARAM(Ref) FChooserEvaluationContext& Context,UPARAM(Ref) const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass = false);
	
	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UE_API TArray<TSoftObjectPtr<UObject>> EvaluateObjectChooserBaseMultiSoft(UPARAM(Ref) FChooserEvaluationContext& Context,UPARAM(Ref) const FInstancedStruct& ObjectChooser, TSubclassOf<UObject> ObjectClass, bool bResultIsClass = false);

	/**
	* Add an Object to a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Object				(in) The Object to add
	*/
	UFUNCTION(BlueprintCallable, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true"))
	static UE_API void AddChooserObjectInput(UPARAM(Ref) FChooserEvaluationContext& Context, UObject* Object);
	
	/**
	* Get an Object from a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Index				(in) the context index to get the object from
	*/
	UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe ,BlueprintInternalUseOnly = "true", DeterminesOutputType = "ObjectClass"))
	static UObject* GetChooserObjectInput(UPARAM(Ref) FChooserEvaluationContext& Context, int32 Index, TSubclassOf<UObject> ObjectClass);

	/**
	* Get an Object from a ChooserEvaluation context by class or interface
	* Experimental, this feature might be removed without warning, not for production use
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param ObjectClass		(in) the wanted object class or interface
	*/
	UFUNCTION(BlueprintPure, Category = "Animation | Experimental", meta = (BlueprintThreadSafe))
	static UObject* GetChooserObject(UPARAM(Ref) FChooserEvaluationContext& Context, TSubclassOf<UObject> ObjectClass);
	
	/**
	* Add a Struct to a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Object				(in) The Object to add
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", CustomStructureParam = "Value"))
	static UE_API void AddChooserStructInput(UPARAM(Ref) FChooserEvaluationContext& Context, int32 Value);
	
	DECLARE_FUNCTION(execAddChooserStructInput);

	/**
	* Add a Struct with output to a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Object				(in) The Object to add
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", CustomStructureParam = "Value"))
	static UE_API void AddChooserStructInputOutput(UPARAM(Ref) FChooserEvaluationContext& Context, int32 Value);
	
	DECLARE_FUNCTION(execAddChooserStructInputOutput);
	
	/**
	* Get a Struct to a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Object				(in) The Object to add
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", CustomStructureParam = "Value"))
	static UE_API void GetChooserStructOutput(UPARAM(Ref) FChooserEvaluationContext& Context, int Index, int32& Value);
    	
	DECLARE_FUNCTION(execGetChooserStructOutput);

	/**
	* Get a Struct to a ChooserEvaluation context
	*
	* @param Context			(in) A struct reference to the chooser evaluation context
	* @param Object				(in) The Object to add
	*/
	UFUNCTION(BlueprintPure, CustomThunk, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly = "true", ArrayParm = "Value"))
	static UE_API void GetChooserStructOutputArray(UPARAM(Ref) FChooserEvaluationContext& Context, int Index, TArray<int32>& Value);
		
	DECLARE_FUNCTION(execGetChooserStructOutputArray);

	/**
	* Create an EvaluateChooser struct
	*
	* @param Chooser				(in) the ChooserTable asset to evaluate
	*/
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe, BlueprintInternalUseOnly="true", NativeMakeFunc))
	static UE_API FInstancedStruct MakeEvaluateChooser(UChooserTable* Chooser);
	
	
	UFUNCTION(BlueprintCallable, Category = "Animation", meta = (BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static UE_API FChooserEvaluationContext MakeChooserEvaluationContext();
};

#undef UE_API
