// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FChooserIndexArray
{
public:
	struct FIndexData
	{
		UE_DEPRECATED(5.5,"This is a fallback conversion operator for Chooser Column implementations which expect index data to be a uint32.  Please convert Chooser Column implementations to use FIndexData before the next release")
		FIndexData(uint32 InIndex) : Index(InIndex), Cost(0) { }
		
		FIndexData(uint32 InIndex, float InCost) : Index(InIndex), Cost(InCost) {}
	
		UE_DEPRECATED(5.5, "This is a fallback conversion operator for Chooser Column implementations which expect index data to be a uint32.  Please convert Chooser Column implementations to use FIndexData before the next release")
		operator uint32() const { return Index; }
		uint32 Index;
		float Cost;

		bool operator < (const FIndexData& Other) const { return Cost < Other.Cost; }
	};
	
	FChooserIndexArray(FIndexData* InData, uint32 InMaxSize): Data(InData), MaxSize(InMaxSize), Size(0) {}
	
	typedef FIndexData* iterator;
	typedef const FIndexData* const_iterator;
	
	iterator begin() { return Data; }
	const_iterator begin() const { return Data; }
	iterator end() { return Data + Size; }
	const_iterator end() const { return Data + Size; }
	
	
	void Push(const FIndexData& Value)
	{
		check(Size < MaxSize);
		Data[Size++] = Value;
	}

	bool IsEmpty() const
	{
		return Size == 0;
	}
	
	uint32 Num() const
	{
		return Size;
	}
	
	void SetNum(uint32 Num)
	{
		check(Num <= MaxSize);
		Size = Num;		
	}

	FIndexData& operator [] (uint32 Index)
	{
		check(Index < MaxSize);
		return Data[Index];
	}
	
	const FIndexData& operator [] (uint32 Index) const
	{
		check(Index < MaxSize);
		return Data[Index];
	}
	
	void operator = (const FChooserIndexArray& Other)
	{
		check(MaxSize >= Other.Size);
		Size = Other.Size;
		FMemory::Memcpy(Data,Other.Data, Size*sizeof(FIndexData));
	}

	FIndexData* GetData() { return Data; }

	bool HasCosts() const { return bHasCosts; };
	void SetHasCosts() { bHasCosts = true; };
private:
	FIndexData* Data;
	uint32 MaxSize;
	uint32 Size;
	bool bHasCosts = false;
};

inline FChooserIndexArray::FIndexData* GetData(FChooserIndexArray& Array)
{
	return Array.GetData();
}

inline uint32 GetNum(FChooserIndexArray& Array)
{
	return Array.Num();
}

