// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterBase.h"
#include "ChooserParameterName.generated.h"

class UObject;

USTRUCT()
struct FChooserParameterNameBase : public FChooserParameterBase
{
	GENERATED_BODY()

	virtual bool GetValue(FChooserEvaluationContext& Context, FName& OutResult) const { return false; }
	virtual bool SetValue(FChooserEvaluationContext& Context, const FName& InValue) const { return false; }

#if WITH_EDITOR
	virtual FLinearColor GetIconColor() const override
	{
		return FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);
	}
#endif
};
