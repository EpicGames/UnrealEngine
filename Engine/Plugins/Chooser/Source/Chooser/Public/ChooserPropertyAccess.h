// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "IHasContext.h"
#include "StructUtils/InstancedStruct.h"
#include "IObjectChooser.h"
#include "ChooserPropertyAccess.generated.h"

#define UE_API CHOOSER_API

#if WITH_EDITOR
struct FBindingChainElement;
#endif


namespace UE::Chooser
{
	struct FCompiledBindingElement
	{
		FCompiledBindingElement() : bIsFunction(false), Offset(0)
		{
		}
		
		explicit FCompiledBindingElement(int InOffset)
		{
			bIsFunction = false;
			Offset = InOffset;
			Mask = 0;
		}
		
		explicit FCompiledBindingElement(UFunction* InFunction)
		{
			bIsFunction = true;
			Function = InFunction;
		}
		
		bool bIsFunction;
		union 
		{
			struct
			{
				int Offset;
				uint8 Mask; // mask, for bitset bools
			};
			UFunction* Function;
		};
	};

	// property type, for numerical conversions
	enum class EChooserPropertyAccessType
	{
		None,
		Bool,
		Int32,
		Float,
		Double,
		SoftObjectRef,
	};

	struct FCompiledBinding
	{
		int ContextIndex = 0;
		EChooserPropertyAccessType PropertyType = EChooserPropertyAccessType::None;
		// type info for number and enum conversions
		TArray<FCompiledBindingElement> CompiledChain;
		const UStruct* TargetType = nullptr;

		// Type of struct for when the property itself is a struct
		const UStruct* StructType = nullptr;
#if WITH_EDITORONLY_DATA
		int SerialNumber = 0;
		TArray<const UStruct*> Dependencies;
#endif
	};
}


USTRUCT()
struct FChooserPropertyBinding 
{
	GENERATED_BODY()

	virtual ~FChooserPropertyBinding() = default;
	
	UPROPERTY()
	TArray<FName> PropertyBindingChain;
	
	UPROPERTY()
	int ContextIndex = 0;
	
	UPROPERTY()
	bool IsBoundToRoot = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString DisplayName;

	FText CompileMessage;

	virtual void SetPropertyData(const IHasContextClass* HasContext, FField* Property) {}
#endif

	TSharedPtr<UE::Chooser::FCompiledBinding> CompiledBinding;

	UE_API FName GetUniqueId() const;

	UE_API void Compile(IHasContextClass* HasContext, bool bForce = false);
	
	template <typename T>
	bool GetValuePtr(FChooserEvaluationContext& Context, T*& Value) const;
	
	template <typename T>
	bool GetStructPtr(FChooserEvaluationContext& Context, T*& Value, UStruct const* &StructType) const;
	
	template <typename T>
	bool GetStructValue(FChooserEvaluationContext& Context, T& Value) const;
	
	template <typename T>
	bool GetValue(FChooserEvaluationContext& Context, T& Value) const;

	template <typename T>
	typename TEnableIf<TIsArithmetic<T>::Value, bool>::Type SetValue(FChooserEvaluationContext& Context, const T& Value) const;

	template <typename T>
	typename TEnableIf<!TIsArithmetic<T>::Value, bool>::Type SetValue(FChooserEvaluationContext& Context, const T& Value) const;
};

USTRUCT()
struct FChooserEnumPropertyBinding : public FChooserPropertyBinding
{
	GENERATED_BODY()
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<const UEnum> Enum = nullptr;

	UE_API virtual void SetPropertyData(const IHasContextClass* HasContext, FField* Property) override;
#endif
};

USTRUCT()
struct FChooserObjectPropertyBinding : public FChooserPropertyBinding
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UClass> AllowedClass = nullptr;
	
	UE_API virtual void SetPropertyData(const IHasContextClass* HasContext, FField* Property) override;
#endif
};

USTRUCT()
struct FChooserStructPropertyBinding : public FChooserPropertyBinding
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UScriptStruct> StructType = nullptr;
	
	UE_API virtual void SetPropertyData(const IHasContextClass* HasContext, FField* Property) override;
#endif
};

UENUM()
enum class EContextObjectDirection
{
	Read UMETA(DisplayName="Input", Tooltip = "This Parameter will only be read from"),
	Write UMETA(DisplayName="Output", Tooltip = "This Parameter will only be written to"),
	ReadWrite UMETA(DisplayName="Input/Output", Tooltip = "This Parameter can be both read from and written to"),
};

USTRUCT()
struct FContextObjectTypeBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Type", meta = (ToolTip="Select weather this property will be read from, written to, or both"))
	EContextObjectDirection Direction = EContextObjectDirection::Read;
};


USTRUCT(DisplayName="Class Parameter")
struct FContextObjectTypeClass : public FContextObjectTypeBase
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category="Type")
	TObjectPtr<UClass> Class;
};

USTRUCT(DisplayName = "Struct Parameter")
struct FContextObjectTypeStruct : public FContextObjectTypeBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Type")
	TObjectPtr<UScriptStruct> Struct;
};

namespace UE::Chooser
{
#if WITH_EDITORONLY_DATA
	CHOOSER_API void RuntimeValidateContext(const UObject* Asset, const TArray<FInstancedStruct>& ContextData, FChooserEvaluationContext& Context);
#define VALIDATE_CHOOSER_CONTEXT(Asset,ContextData,Context) UE::Chooser::RuntimeValidateContext(Asset, ContextData, Context)
#else
#define VALIDATE_CHOOSER_CONTEXT(Asset,ContextData,Context)
#endif

	struct FResolvedPropertyChainResult
	{
		uint8* Container = nullptr;
		uint32 PropertyOffset = 0;
		UFunction* Function = nullptr;
		const UStruct* StructType = nullptr;
		EChooserPropertyAccessType PropertyType = EChooserPropertyAccessType::None;
		uint8 Mask = 0;
	};
	
	CHOOSER_API bool ResolvePropertyChain(FChooserEvaluationContext& Context, const FChooserPropertyBinding& Binding, FResolvedPropertyChainResult& Result);

#if WITH_EDITOR
	CHOOSER_API void CopyPropertyChain(const TArray<FBindingChainElement>& InBindingChain, FChooserPropertyBinding& OutPropertyBinding);
#endif
}

template <typename T>
bool FChooserPropertyBinding::GetValuePtr(FChooserEvaluationContext& Context, T*& OutResult) const
{
	using namespace  UE::Chooser;

	FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function == nullptr)
		{
			OutResult = reinterpret_cast<T*>(Result.Container + Result.PropertyOffset);
			return true;
		}
	}
	return false;
}

template <typename T>
bool FChooserPropertyBinding::GetStructPtr(FChooserEvaluationContext& Context, T*& OutResult, UStruct const* &OutStructType) const
{
	using namespace  UE::Chooser;

	FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function == nullptr)
		{
			OutResult = reinterpret_cast<T*>(Result.Container + Result.PropertyOffset);
			OutStructType = Result.StructType;
			return true;
		}
	}
	return false;
}

template <typename T>
bool FChooserPropertyBinding::GetStructValue(FChooserEvaluationContext& Context, T& OutResult) const
{
	UE::Chooser::FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function)
		{
			UObject* Object = reinterpret_cast<UObject*>(Result.Container);
			Object->ProcessEvent(Result.Function, &OutResult);
			return true;
		}
		else
		{
			OutResult = *reinterpret_cast<T*>(Result.Container + Result.PropertyOffset);
			return true;
		}
	}
	
	return false;
	}

template <typename T>
bool FChooserPropertyBinding::GetValue(FChooserEvaluationContext& Context, T& OutResult) const
{
	using namespace  UE::Chooser;

	FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function == nullptr)
		{
			switch (Result.PropertyType)
			{
			case EChooserPropertyAccessType::Float:
				{
					float FloatResult = *reinterpret_cast<float*>(Result.Container + Result.PropertyOffset);
					OutResult = static_cast<T>(FloatResult);
					break;
				}
			case EChooserPropertyAccessType::Double:
				{
					double DoubleResult = *reinterpret_cast<double*>(Result.Container + Result.PropertyOffset);
					OutResult = static_cast<T>(DoubleResult);
					break;
				}
			case EChooserPropertyAccessType::Int32:
				{
					int32 IntResult = *reinterpret_cast<int*>(Result.Container + Result.PropertyOffset);
					OutResult = static_cast<T>(IntResult);
					break;
				}
			case EChooserPropertyAccessType::Bool:
				{
					uint8* ByteValue = Result.Container + Result.PropertyOffset;
					OutResult = !!(*ByteValue & Result.Mask);
					break;
				}
			default:
				OutResult = *reinterpret_cast<T*>(Result.Container + Result.PropertyOffset);
				break;
			}
		}
		else
		{
			UObject* Object = reinterpret_cast<UObject*>(Result.Container);
			if (Result.Function->IsNative())
			{
				FFrame Stack(Object, Result.Function, nullptr, nullptr, Result.Function->ChildProperties);
				switch (Result.PropertyType)
				{
				case EChooserPropertyAccessType::Float:
					{
						float FloatResult = 0;
						Result.Function->Invoke(Object, Stack, &FloatResult);
						OutResult = static_cast<T>(FloatResult);
						break;
					}
				case EChooserPropertyAccessType::Double:
					{
						double DoubleResult = 0;
						Result.Function->Invoke(Object, Stack, &DoubleResult);
						OutResult = static_cast<T>(DoubleResult);
						break;
					}
				case EChooserPropertyAccessType::Int32:
					{
						int32 IntResult = 0;
						Result.Function->Invoke(Object, Stack, &IntResult);
						OutResult = static_cast<T>(IntResult);
						break;
					}
				default:
					Result.Function->Invoke(Object, Stack, &OutResult);
					break;
				}
			}
			else
			{
				switch (Result.PropertyType)
				{
				case EChooserPropertyAccessType::Float:
					{
						float FloatResult = 0;
						Object->ProcessEvent(Result.Function, &FloatResult);
						OutResult = static_cast<T>(FloatResult);
						break;
					}
				case EChooserPropertyAccessType::Double:
					{
						double DoubleResult = 0;
						Object->ProcessEvent(Result.Function, &DoubleResult);
						OutResult = static_cast<T>(DoubleResult);
						break;
					}
				case EChooserPropertyAccessType::Int32:
					{
						int32 IntResult = 0;
						Object->ProcessEvent(Result.Function, &IntResult);
						OutResult = static_cast<T>(IntResult);
						break;
					}
				default:
					Object->ProcessEvent(Result.Function, &OutResult);
					break;
				}
			}
		}
		return true;
	}
	return false;
}

template <typename T>
typename TEnableIf<TIsArithmetic<T>::Value, bool>::Type FChooserPropertyBinding::SetValue(FChooserEvaluationContext& Context, const T& Value) const
{
	using namespace  UE::Chooser;

	FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function == nullptr)
		{
			switch (Result.PropertyType)
			{
			case EChooserPropertyAccessType::None:
			{
				*reinterpret_cast<T*>(Result.Container + Result.PropertyOffset) = Value;
				break;
			}
			case EChooserPropertyAccessType::Float:
				{
					*reinterpret_cast<float*>(Result.Container + Result.PropertyOffset) = static_cast<float>(Value);
					break;
				}
			case EChooserPropertyAccessType::Double:
				{
					*reinterpret_cast<double*>(Result.Container + Result.PropertyOffset) = static_cast<double>(Value);
					break;
				}
			case EChooserPropertyAccessType::Int32:
				{
					*reinterpret_cast<int*>(Result.Container + Result.PropertyOffset) = static_cast<int>(Value);
					break;
				}
			case EChooserPropertyAccessType::Bool:
			{
				uint8* ByteValue = Result.Container + Result.PropertyOffset;
				if (Result.Mask == 255) // regular bool
				{
					*reinterpret_cast<bool*>(Result.Container + Result.PropertyOffset) = static_cast<bool>(Value);
				}
				else // bitset bool
				{
					*ByteValue = ((*ByteValue) & ~Result.Mask) | (Value ? Result.Mask : 0);
				}
				break;
			}
			default:
				*reinterpret_cast<T*>(Result.Container + Result.PropertyOffset) = Value;
				break;
			}
			return true;
		}
	}
	return false;
}

template <typename T>
typename TEnableIf<!TIsArithmetic<T>::Value, bool>::Type FChooserPropertyBinding::SetValue(FChooserEvaluationContext& Context, const T& Value) const
{
	using namespace  UE::Chooser;

	FResolvedPropertyChainResult Result;
	if (ResolvePropertyChain(Context, *this, Result))
	{
		if (Result.Function == nullptr)
		{
			*reinterpret_cast<T*>(Result.Container + Result.PropertyOffset) = Value;
			return true;
		}
	}
	return false;
}

#if WITH_EDITOR
#define CHOOSER_PARAMETER_BOILERPLATE() \
	virtual void Compile(IHasContextClass* Owner, bool bForce) override\
	{\
		Binding.Compile(Owner, bForce);\
	};\
	virtual bool HasCompileErrors(FText& Message) const override\
	{\
		Message = Binding.CompileMessage; \
		return !Message.IsEmpty(); \
	}\
	virtual void AddSearchNames(FStringBuilderBase& Builder) const override\
	{\
		for (const FName& Entry : Binding.PropertyBindingChain)\
		{\
			Builder.Append(Entry.ToString());\
			Builder.Append(";");\
		}\
	}\
	\
	virtual void GetDisplayName(FText& OutName) const override\
	{\
		if (!Binding.DisplayName.IsEmpty())\
		{\
			OutName = FText::FromString(Binding.DisplayName);\
		} \
		else if (!Binding.PropertyBindingChain.IsEmpty())\
		{\
			OutName = FText::FromName(Binding.PropertyBindingChain.Last());\
		}\
	}\
	\
	virtual void ReplaceString(FStringView FindString, ESearchCase::Type SearchCase, bool bFindWholeWord, FStringView ReplaceString) override\
	{\
		for (FName& Entry : Binding.PropertyBindingChain)\
		{\
			if(bFindWholeWord)\
			{\
				if(Entry.ToString().Compare(FString(FindString), SearchCase) == 0)\
				{\
					Entry = FName(ReplaceString);\
					Binding.DisplayName = "";\
				}\
			}\
			else\
			{\
				if(UE::String::FindFirst(Entry.ToString(), FindString, SearchCase) != INDEX_NONE)\
				{\
					FString NewString = Entry.ToString().Replace(FindString.begin(), ReplaceString.begin(), SearchCase);\
					Entry = FName(NewString);\
					Binding.DisplayName = "";\
				}\
			}\
		}\
	}
#elif WITH_EDITORONLY_DATA
#define CHOOSER_PARAMETER_BOILERPLATE() \
	virtual void Compile(IHasContextClass* Owner, bool bForce) override\
	{\
		Binding.Compile(Owner, bForce);\
	}\
	virtual bool HasCompileErrors(FText& Message) const override\
	{\
		Message = Binding.CompileMessage; \
		return !Message.IsEmpty(); \
	}
#else
#define CHOOSER_PARAMETER_BOILERPLATE() \
	virtual void Compile(IHasContextClass* Owner, bool bForce) override\
	{\
		Binding.Compile(Owner, bForce);\
	}\
	virtual bool HasCompileErrors(FText& Message) const override\
	{\
		return false;\
	}
#endif

#undef UE_API
