// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IObjectChooser.h"
#include "StructUtils/InstancedStruct.h"
#include "IHasContext.h"
#include "ChooserSignature.generated.h"

/**
* Data table used to choose an asset based on input parameters
*/
UCLASS(MinimalAPI, BlueprintType)
class UChooserSignature : public UObject, public IHasContextClass
{
	GENERATED_UCLASS_BODY()
public:
	UChooserSignature() {}
	
	// The kind of output this chooser has (Object or Class or No primary result)
    UPROPERTY(EditAnywhere, DisplayName = "Result Type", Category="Result", Meta = (EditConditionHides, EditCondition="ResultType != EObjectChooserResultType::NoPrimaryResult"))
    EObjectChooserResultType ResultType = EObjectChooserResultType::ObjectResult;

	// The Class of Object this Chooser returns when ResultType is set to ObjectOfType, or the Parent Class of the Classes returned by this chooser when ResultType is set to ClassOfType
	UPROPERTY(EditAnywhere, DisplayName= "Result Class", Category="Result", Meta = (AllowAbstract=true, EditConditionHides, EditCondition="ResultType != EObjectChooserResultType::NoPrimaryResult"))
	TObjectPtr<UClass> OutputObjectType;
	
	// Parameter Objects or Structs from which the chooser can read or write properties 
	UPROPERTY(EditAnywhere, NoClear, DisplayName = "Parameters", Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ContextObjectTypeBase"), Category = "Parameters")
	TArray<FInstancedStruct> ContextData;
	
	// IHasContextClass implementation
	virtual TConstArrayView<FInstancedStruct> GetContextData() const override { return ContextData; }
	virtual FString GetContextOwnerName() const override { return GetName(); }
	virtual UObject* GetContextOwnerAsset() override { return this; }
};


