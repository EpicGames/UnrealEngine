// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChooserIndexArray.h"
#include "CoreTypes.h"
#include "Trace/Config.h"
#include "TraceFilter.h"
#include "Misc/Variant.h"
#include "IObjectChooser.h"
#include "Serialization/BufferArchive.h"

#define UE_API CHOOSER_API

#if CHOOSER_TRACE_ENABLED

class UObject;

struct FChooserEvaluationContext;

struct FChooserTrace
{
	static UE_API void OutputChooserEvaluation(const UObject* ChooserAsset, const FChooserEvaluationContext& Context, int32 SelectedIndex);
	static UE_API void OutputChooserEvaluation(const UObject* ChooserAsset, const FChooserEvaluationContext& Context, const FChooserIndexArray& SelectedIndex);
	static UE_API void OutputChooserValueArchive(const FChooserEvaluationContext& Context, const TCHAR* Key, const FBufferArchive& ValueArchive);
	
	template<typename T>
	static void OutputChooserValue(const FChooserEvaluationContext& Context, const TCHAR* Key, const T& Value)
	{
		FBufferArchive Archive;
		Archive << const_cast<T&>(Value);
		OutputChooserValueArchive(Context,Key,Archive);
	}
};

#define TRACE_CHOOSER_EVALUATION(Chooser, Context, SelectedIndex) \
	FChooserTrace::OutputChooserEvaluation(Chooser, Context, SelectedIndex)
	
#define TRACE_CHOOSER_VALUE(Context, Key, Value) \
	FChooserTrace::OutputChooserValue(Context, Key, Value);
#else

#define TRACE_CHOOSER_EVALUATION(...)
#define TRACE_CHOOSER_VALUE(...)

#endif

#undef UE_API
