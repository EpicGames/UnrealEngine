// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "Animation/BlendProfile.h"
#include "ChooserTypes.generated.h"


UENUM()
enum class EChooserEvaluationFrequency
{
	// Evaluate the Chooser once the Node starts updating for the first time
	OnInitialUpdate,
	// Evaluate the Chooser when the Node starts updating
	OnBecomeRelevant,
	// Evaluate the Chooser when the animation it's playing has looped
	OnLoop,
	// Evaluate the Chooser every update
	OnUpdate
};


USTRUCT(BlueprintType)
struct FAnimCurveOverride
{
	GENERATED_BODY()
	// Name of curve to override
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CurveValue)
	FName CurveName;
	// Value to set to the curve
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CurveValue)
	float CurveValue = 0.0f;
};

template <>
struct TTypeTraits<FAnimCurveOverride> : public TTypeTraitsBase < FAnimCurveOverride >
{
	enum { IsBytewiseComparable = true };
};

USTRUCT(BlueprintType)
struct FAnimCurveOverrideList
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Values, meta = (ExpandByDefault = true))
	TArray<FAnimCurveOverride> Values;

	UPROPERTY()
	uint32 Hash = 0;

	CHOOSER_API void ComputeHash();
};

USTRUCT(BlueprintType)
struct FChooserPlayerSettings
{
	GENERATED_BODY()

	bool Serialize(FArchive& Ar);
	bool Serialize(FStructuredArchive::FSlot Slot);

	// Set this value to mirror animations - the MirrorDataTable must also be set on the AnimNode
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bMirror = false;

	// Start offset when starting the Animation Asset
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (FrameTimeEditor))
	float StartTime = 0.f;

	// Loop the animation asset, even if the asset is not set as looping
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bForceLooping = false;

	// playback rate modifier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	float PlaybackRate = 1.f;

	// List of curve values to set 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (ExpandByDefault = true))
	FAnimCurveOverrideList CurveOverrides;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending, meta = (FrameTimeEditor))
	float BlendTime = 0.2f;

	// Set Blend Profiles (editable in the skeleton) to determine how the blending is distributed among your character's bones. It could be used to differentiate between upper body and lower body to blend timing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending, meta = (UseAsBlendProfile = true))
	TObjectPtr<UBlendProfile> BlendProfile;

	// How the blend is applied over time to the bones. Common selections are linear, ease in, ease out, and ease in and out.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending)
	EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending)
	bool bUseInertialBlend = false;

	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending)
	bool bForceBlendTo = false;

	// Experimental, this feature might be removed without warning, not for production use
	// the chooser player will continue playing an asset previously selected at time "PreviousStartTime", integrating it by the simulation "DeltaSeconds",
	// unless the current "StartTime" is far away "MinDeltaTimeToForceBlendTo" from "PreviousStartTime":
	// if |PreviousStartTime - StartTime| > MinDeltaTimeToForceBlendTo then bForceBlendTo = true and the chooser player will ask blend stack to blend to the same asset at a different "StartTime"
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blending, meta = (FrameTimeEditor))
	float MinDeltaTimeToForceBlendTo = 0.1f;

	// Experimental, this feature might be removed without warning, not for production use
	// Currently playing animation
	UPROPERTY(Transient)
	TObjectPtr<const UObject> PlayingAsset = nullptr;

	// Experimental, this feature might be removed without warning, not for production use
	// Currently playing animation accumulated time
	UPROPERTY(Transient)
	float PlayingAssetAccumulatedTime = 0.f;

	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY(Transient)
	bool bIsPlayingAssetMirrored = false;

	// Experimental, this feature might be removed without warning, not for production use
	// PlayingAsset associated BlendParameters (if PlayingAsset is a UBlendSpace)
	UPROPERTY(Transient)
	FVector PlayingAssetBlendParameters = FVector::ZeroVector;

	// Experimental, this feature might be removed without warning, not for production use
	const struct FAnimationUpdateContext* AnimationUpdateContext = nullptr;
};

template<> struct TStructOpsTypeTraits<FChooserPlayerSettings> : public TStructOpsTypeTraitsBase2<FChooserPlayerSettings>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializer = true,
	};
};