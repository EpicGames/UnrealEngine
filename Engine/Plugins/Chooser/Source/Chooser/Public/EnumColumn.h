// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterEnum.h"
#include "StructUtils/InstancedStruct.h"
#include "ChooserPropertyAccess.h"
#include "Serialization/MemoryReader.h"
#include "EnumColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

UENUM()
enum class EEnumColumnCellValueComparison
{
	MatchEqual,
	MatchNotEqual,
	MatchAny,

	Modulus // used for cycling through the other values
};

USTRUCT(DisplayName = "Enum Property Binding")
struct FEnumContextProperty : public FChooserParameterEnumBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FName> PropertyBindingChain_DEPRECATED;
	
	UPROPERTY(EditAnywhere, Meta = (BindingType = "enum", BindingColor = "BytePinTypeColor"), Category = "Binding")
	FChooserEnumPropertyBinding Binding;

	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, uint8& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, uint8 InValue) const override;

	virtual void PostLoad() override
	{
		if (PropertyBindingChain_DEPRECATED.Num() > 0)
		{
			Binding.PropertyBindingChain = PropertyBindingChain_DEPRECATED;
			PropertyBindingChain_DEPRECATED.SetNum(0);
#if WITH_EDITORONLY_DATA
			Binding.Enum = Enum_DEPRECATED;
			Enum_DEPRECATED = nullptr;
#endif
		}
	}

	CHOOSER_PARAMETER_BOILERPLATE();

#if WITH_EDITOR
	virtual const UEnum* GetEnum() const override { return Binding.Enum; }
#endif
	
private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<const UEnum> Enum_DEPRECATED = nullptr;
#endif
};

USTRUCT()
struct FChooserEnumRowData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool CompareNotEqual_DEPRECATED = false;

	UPROPERTY(EditAnywhere, Category = EditorOnly)
	FName ValueName;
#endif
	
	UPROPERTY(EditAnywhere, Category = Runtime, Meta = (ValidEnumValues = "MatchEqual, MatchNotEqual, MatchAny"))
	EEnumColumnCellValueComparison Comparison = EEnumColumnCellValueComparison::MatchEqual;
	
	UPROPERTY(EditAnywhere, Category = Runtime)
	uint8 Value = 0;

	bool Evaluate(const uint8 LeftHandSide) const;
};


USTRUCT(Meta=(Hidden))
struct FEnumColumnBase : public FChooserColumnBase
{
	GENERATED_BODY()
	
	virtual void EnumChanged(const UEnum* Enum) {}
};

USTRUCT(DisplayName = "Enum", Meta = (Category = "Filter", Tooltip = "A column that filters rows based on the value of an Enum, with Equal Not Equal, or Any as cell comparison options."))
struct FEnumColumn : public FEnumColumnBase
{
	GENERATED_BODY()
public:
	UE_API FEnumColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterEnumBase", ToolTip="The Enum property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserEnumRowData DefaultRowValue;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserEnumRowData> RowValues;

	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;
	
#if WITH_EDITOR
	const UEnum* GetEnum() const 
	{
		const UEnum* Enum = nullptr;
		if (const FChooserParameterEnumBase* Input = InputValue.GetPtr<FChooserParameterEnumBase>())
		{
			Enum = Input->GetEnum();
		}
		return Enum;
	}
	
	mutable uint8 TestValue = 0;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return RowValues.IsValidIndex(RowIndex) && RowValues[RowIndex].Evaluate(TestValue);
	}
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		Reader << TestValue;
	}
	
	UE_API virtual void AddToDetails (FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;

	UE_API virtual void EnumChanged(const UEnum* Enum) override;
#endif
	
	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterEnumBase);

	UE_API virtual void PostLoad() override;
};

// deprecated class version for converting old data
UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserParameterEnum_ContextProperty : public UObject, public IChooserParameterEnum
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FName> PropertyBindingChain;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FEnumContextProperty::StaticStruct());
		FEnumContextProperty& Property = OutInstancedStruct.GetMutable<FEnumContextProperty>();
		Property.Binding.PropertyBindingChain = PropertyBindingChain;
	}
};

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserColumnEnum : public UObject, public IChooserColumn
{
	GENERATED_BODY()
public:
	UDEPRECATED_ChooserColumnEnum() = default;
	UDEPRECATED_ChooserColumnEnum(const FObjectInitializer& ObjectInitializer)
	{
		InputValue = ObjectInitializer.CreateDefaultSubobject<UDEPRECATED_ChooserParameterEnum_ContextProperty>(this, "InputValue");
	}	

	UPROPERTY(EditAnywhere, Category = "Input")
	TScriptInterface<IChooserParameterEnum> InputValue;

	UPROPERTY(EditAnywhere, Category = Runtime)
	TArray<FChooserEnumRowData> RowValues;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FEnumColumn::StaticStruct());
		FEnumColumn& Column = OutInstancedStruct.GetMutable<FEnumColumn>();
		if (IChooserParameterEnum* InputValueInterface = InputValue.GetInterface())
		{
			InputValueInterface->ConvertToInstancedStruct(Column.InputValue);
		}
		Column.RowValues = RowValues;
	}
};

#undef UE_API
