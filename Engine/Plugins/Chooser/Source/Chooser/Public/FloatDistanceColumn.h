// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <IHasContext.h>

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterFloat.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "Templates/SubclassOf.h"
#include "FloatDistanceColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT()
struct FChooserFloatDistanceRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Runtime)
	float Value=0;

#if WITH_EDITORONLY_DATA
	// Set this property to lock the current cell value, preventing it from being updated by AutoPopulate
	UPROPERTY(EditAnywhere, Category=Runtime)
	bool DisableAutoPopulate = false;
#endif
};

UCLASS(MinimalAPI)
class UFloatAutoPopulator : public UObject
{
	GENERATED_BODY()
public:
	virtual void NativeAutoPopulate(UObject* InObject, bool& OutSuccess, float& OutValue) { }
};

USTRUCT(DisplayName = "Float Difference", Meta = (Category = "Scoring", Tooltip = "A column which scores rows based on their difference from an Input float."))
struct FFloatDistanceColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FFloatDistanceColumn();
		
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterFloatBase", ToolTip="The Float column this fillter will Score based on"), Category = "Data")
	FInstancedStruct InputValue;

	// Maximum Distance used for normalizing scoring (greater distances will be considered equal to the max)
	UPROPERTY(EditAnywhere, Category = "Data");
	double MaxDistance= 100;

	// Multiplier for controlling which scoring column has the most influence.  Higher values will make the match from this column more important.
	UPROPERTY(EditAnywhere, Category = "Data");
	float CostMultiplier = 1;

	// For rows with distance greater than MaxDistance, filter out the row
	UPROPERTY(EditAnywhere, Category = "Data");
	bool bFilterOverMaxDistance = false;
	// Wrap input, and distance calculations for numbers such as angles
	UPROPERTY(EditAnywhere, Category = "Data");
	bool bWrapInput = false;
	
	// Minimum value (for WrapInput)
	UPROPERTY(EditAnywhere, Category = "Data", meta=(DisplayAfter="bWrapInput", EditCondition="bWrapInput"));
	double MinValue = -180;
	
	// Maximum value (for WrapInput)
	UPROPERTY(EditAnywhere, Category = "Data", meta=(DisplayAfter="bWrapInput", EditCondition="bWrapInput"));
	double MaxValue = 180;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserFloatDistanceRowData DefaultRowValue;

	// Optional class to auto populate column data based on the result asset
	UPROPERTY(EditAnywhere, Category = "Data")
	TSubclassOf<UFloatAutoPopulator> AutoPopulator;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FChooserFloatDistanceRowData> RowValues;

	virtual bool HasFilters() const { return true; }
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

	virtual bool HasCosts() const override { return true; }

#if WITH_EDITOR
	mutable double TestValue = 0.0;
	UE_API virtual bool EditorTestFilter(int32 RowIndex) const override;
	UE_API virtual float EditorTestCost(int32 RowIndex) const override;
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		Reader << TestValue;
	}

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;

	virtual bool AutoPopulates() const override { return AutoPopulator != nullptr; }
	UE_API virtual void AutoPopulate(int32 RowIndex, UObject* OutputObject) override;
#endif
	
	virtual void PostLoad() override
	{
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterFloatBase);

};

#undef UE_API
