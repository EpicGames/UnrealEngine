// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <IHasContext.h>

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterFloat.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "FloatRangeColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Float Property Binding")
struct FFloatContextProperty :  public FChooserParameterFloatBase
{
	GENERATED_BODY()
	
	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, double& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, double Value) const override;

	UPROPERTY()
	TArray<FName> PropertyBindingChain_DEPRECATED;
	
	UPROPERTY(EditAnywhere, Meta = (BindingType = "double", BindingAllowFunctions = "true", BindingColor = "FloatPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;

	virtual void PostLoad() override
	{
		if (PropertyBindingChain_DEPRECATED.Num() > 0)
		{
			Binding.PropertyBindingChain = PropertyBindingChain_DEPRECATED;
			PropertyBindingChain_DEPRECATED.SetNum(0);
		}
	}
	
	CHOOSER_PARAMETER_BOILERPLATE();
};

USTRUCT()
struct FChooserFloatRangeRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, meta=(DisplayAfter="bNoMin", EditCondition="bNoMin==false"), Category=Runtime)
	float Min=0;
	
	UPROPERTY(EditAnywhere, meta=(DisplayAfter="bNoMax", EditCondition="bNoMax==false"), Category=Runtime)
	float Max=0;

	// Infinite minimum range
	UPROPERTY(EditAnywhere, Category=Runtime)
   	bool bNoMin=false;
	
	// Infinite maximum range
	UPROPERTY(EditAnywhere, Category=Runtime)
   	bool bNoMax=false;
};


USTRUCT(DisplayName = "Float Range", Meta = (Category = "Filter", Tooltip = "A column which filters rows if an input value is not within the range specified for the row."))
struct FFloatRangeColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FFloatRangeColumn();
		
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterFloatBase", ToolTip="The Float property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

	// Wrap input, and comparisons for numbers such as angles which 
	UPROPERTY(EditAnywhere, Category = "Data");
	bool bWrapInput = false;
	// Minimum value (for WrapInput)
	UPROPERTY(EditAnywhere, Category = "Data", meta=(DisplayAfter="bWrapInput", EditCondition="bWrapInput"));
	double MinValue = -180;
	// Maximum value (for WrapInput)
	UPROPERTY(EditAnywhere, Category = "Data", meta=(DisplayAfter="bWrapInput", EditCondition="bWrapInput"));
	double MaxValue = 180;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserFloatRangeRowData DefaultRowValue;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FChooserFloatRangeRowData> RowValues;
	
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable double TestValue = 0.0;
	UE_API virtual bool EditorTestFilter(int32 RowIndex) const override;
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		Reader << TestValue;
	}

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif
	
	virtual void PostLoad() override
	{
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterFloatBase);

};

// deprecated class version to support upgrading old data

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserParameterFloat_ContextProperty :  public UObject, public IChooserParameterFloat
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FName> PropertyBindingChain;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FFloatContextProperty::StaticStruct());
		FFloatContextProperty& Property = OutInstancedStruct.GetMutable<FFloatContextProperty>();
		Property.Binding.PropertyBindingChain = PropertyBindingChain;
	}
};

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserColumnFloatRange : public UObject, public IChooserColumn
{
	GENERATED_BODY()
	public:
	UDEPRECATED_ChooserColumnFloatRange() {}
	UDEPRECATED_ChooserColumnFloatRange(const FObjectInitializer& ObjectInitializer)
	{
		InputValue = ObjectInitializer.CreateDefaultSubobject<UDEPRECATED_ChooserParameterFloat_ContextProperty>(this, "InputValue");
	}	
		
	UPROPERTY(EditAnywhere, Category = "Input")
	TScriptInterface<IChooserParameterFloat> InputValue;
	
	UPROPERTY(EditAnywhere, Category=Runtime)
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FChooserFloatRangeRowData> RowValues;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FFloatRangeColumn::StaticStruct());
		FFloatRangeColumn& Column = OutInstancedStruct.GetMutable<FFloatRangeColumn>();
		if (IChooserParameterFloat* InputValueInterface = InputValue.GetInterface())
		{
			InputValueInterface->ConvertToInstancedStruct(Column.InputValue);
		}
		Column.RowValues = RowValues;
	}
};

#undef UE_API
