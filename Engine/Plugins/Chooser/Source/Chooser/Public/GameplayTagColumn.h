// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterGameplayTag.h"
#include "ChooserPropertyAccess.h"
#include "GameplayTagContainer.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "GameplayTagColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;


USTRUCT(DisplayName = "Gameplay Tags Property Binding")
struct FGameplayTagContextProperty :  public FChooserParameterGameplayTagBase
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FName> PropertyBindingChain_DEPRECATED;

	UPROPERTY(EditAnywhere, NoClear, Meta = (BindingType = "FGameplayTagContainer", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;

	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer*& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer& Value) const override;

	virtual void PostLoad() override
	{
		if (PropertyBindingChain_DEPRECATED.Num() > 0)
		{
			Binding.PropertyBindingChain = PropertyBindingChain_DEPRECATED;
			PropertyBindingChain_DEPRECATED.SetNum(0);
		}
	}

	CHOOSER_PARAMETER_BOILERPLATE();
};

UENUM()
enum class EGameplayTagMatchDirection : uint8
{
	// Row will pass if the Column Input Tags contains the Row Tag(s)
	RowValueInInput,
	// Row will pass if the Row Tags contain the Column Input Tag(s)
	InputInRowValue,
};

USTRUCT(DisplayName = "Gameplay Tag", Meta = (Category = "Filter", Tooltip = "A column which filters rows by comparing Gameplay Tags."))
struct FGameplayTagColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FGameplayTagColumn();
	
	UPROPERTY(EditAnywhere, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterGameplayTagBase", ToolTip="The GameplayTagCollection property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

	UPROPERTY(EditAnywhere, Category="Data")
	EGameplayContainerMatchType	TagMatchType = EGameplayContainerMatchType::Any;

	
	UPROPERTY(EditAnywhere, Category="Data")
	EGameplayTagMatchDirection	TagMatchDirection = EGameplayTagMatchDirection::RowValueInInput;

	//	If true, leaf tags must match exactly.
	UPROPERTY(EditAnywhere, Category="Data")
	bool bMatchExact = false;

	//	If true, rows that pass the normal tag filter will be rejected, and vice versa
	UPROPERTY(EditAnywhere, Category="Data")
	bool bInvertMatchingLogic = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FGameplayTagContainer DefaultRowValue;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FGameplayTagContainer> RowValues;

	UE_API bool TestRow(int32 RowIndex, const FGameplayTagContainer& Value) const;
	
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable FGameplayTagContainer TestValue;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return TestRow(RowIndex, TestValue);
	}

	virtual void SetTestValue(TArrayView<const uint8> Value) override
    {
    	FMemoryReaderView Reader(Value);
		FString Tags;
    	Reader << Tags;
		TestValue.FromExportString(Tags);
    }
	
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	virtual void PostLoad() override
	{
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterGameplayTagBase);
};

// deprecated class versions for converting old data

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserParameterGameplayTag_ContextProperty :  public UObject, public IChooserParameterGameplayTag
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FName> PropertyBindingChain;
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FGameplayTagContextProperty::StaticStruct());
		FGameplayTagContextProperty& Property = OutInstancedStruct.GetMutable<FGameplayTagContextProperty>();
		Property.Binding.PropertyBindingChain = PropertyBindingChain;
	}
};

UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ChooserColumnGameplayTag : public UObject, public IChooserColumn
{
	GENERATED_BODY()
	public:
	UDEPRECATED_ChooserColumnGameplayTag() {};
	UDEPRECATED_ChooserColumnGameplayTag(const FObjectInitializer& ObjectInitializer)
	{
		InputValue = ObjectInitializer.CreateDefaultSubobject<UDEPRECATED_ChooserParameterGameplayTag_ContextProperty>(this, "InputValue");
	}	
	
	UPROPERTY(EditAnywhere, Category = "Input")
	TScriptInterface<IChooserParameterGameplayTag> InputValue;

	UPROPERTY(EditAnywhere, Category=Runtime)
	EGameplayContainerMatchType	TagMatchType = EGameplayContainerMatchType::Any;

	UPROPERTY(EditAnywhere, Category=Runtime)
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FGameplayTagContainer> RowValues;

	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FGameplayTagColumn::StaticStruct());
		FGameplayTagColumn& Column = OutInstancedStruct.GetMutable<FGameplayTagColumn>();
		if (IChooserParameterGameplayTag* InputValueInterface = InputValue.GetInterface())
		{
			InputValueInterface->ConvertToInstancedStruct(Column.InputValue);
		}
		Column.RowValues = RowValues;
	}
};

#undef UE_API
