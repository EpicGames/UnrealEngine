// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IChooserColumn.h"
#include "IChooserParameterGameplayTag.h"
#include "ChooserPropertyAccess.h"
#include "GameplayTagContainer.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "GameplayTagQueryColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Gameplay Tag Query", Meta = (Category = "Filter", Tooltip = "A column which filters rows by matching the input against a Gameplay Tag Query."))
struct FGameplayTagQueryColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FGameplayTagQueryColumn();
	
	UPROPERTY(EditAnywhere, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterGameplayTagBase", ToolTip="The GameplayTagContainer property this column will filter by apply the query on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FGameplayTagQuery DefaultRowValue;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array 
	TArray<FGameplayTagQuery> RowValues;

	UE_API bool TestRow(int32 RowIndex, const FGameplayTagContainer& Value) const;
	
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable FGameplayTagContainer TestValue;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return TestRow(RowIndex, TestValue);
	}

	virtual void SetTestValue(TArrayView<const uint8> Value) override
    {
    	FMemoryReaderView Reader(Value);
		FString Tags;
    	Reader << Tags;
		TestValue.FromExportString(Tags);
    }
	
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	virtual void PostLoad() override
	{
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterGameplayTagBase);
};

#undef UE_API
