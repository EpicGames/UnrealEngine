// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "IObjectChooser.h"
#include "IChooserParameterBase.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserColumn.generated.h"

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UChooserColumn : public UInterface
{
	GENERATED_BODY()
};

class IChooserColumn 
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const {}
};

class UChooserTable;
class FVariant;

class FChooserIndexArray;

struct FInstancedPropertyBag;

USTRUCT()
struct FChooserColumnBase
{
	GENERATED_BODY()

public:
	virtual ~FChooserColumnBase() = default;
	virtual void PostLoad() {}

	virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const {}
	
	// Experimental, this feature might be removed without warning, not for production use
	virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut, TArrayView<uint8> ScratchArea) const
	{
		Filter(Context, IndexListIn, IndexListOut);
	}

	virtual bool HasCosts() const { return false; }
	virtual bool HasFilters() const { return true; }
	virtual bool HasOutputs() const { return false; }
	
	virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const { }
	
	// Experimental, this feature might be removed without warning, not for production use
	virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex, TArrayView<uint8> ScratchArea) const
	{
		SetOutputs(Context, RowIndex);
	}

	virtual void Compile(IHasContextClass* Owner, bool bForce)
	{
#if WITH_EDITORONLY_DATA
		if (bDisabled)
		{
			return;
		}
#endif
		if (FChooserParameterBase* Input = GetInputValue())
		{
			Input->Compile(Owner, bForce);
		}
	};

	virtual void SetTestValue(TArrayView<const uint8> Value) {}
	virtual FChooserParameterBase* GetInputValue() { return nullptr; }
	virtual FInstancedStruct* GetInputValuePtr() { return nullptr; }

	// Experimental, this feature might be removed without warning, not for production use
	virtual int32 GetScratchAreaSize() const { return 0; }
	// Experimental, this feature might be removed without warning, not for production use
	virtual void InitializeScratchArea(TArrayView<uint8> ScratchArea) const {}
	// Experimental, this feature might be removed without warning, not for production use
	virtual void DeinitializeScratchArea(TArrayView<uint8> ScratchArea) const {}

#if WITH_EDITOR
	virtual void Initialize(UChooserTable* OuterChooser) {}
	
	virtual FName RowValuesPropertyName() { return FName(); }
	virtual void SetNumRows(int32 NumRows) {}
	virtual void DeleteRows(TArrayView<int> RowIndices) {}
	virtual void MoveRow(int SourceIndex, int TargetIndex) {}
	virtual void InsertRows(int Index, int Count) {}
	virtual void CopyRow(FChooserColumnBase& SourceColumn, int SourceIndex, int TargetIndex) {}
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) {}
	
	virtual UScriptStruct* GetInputBaseType() const { return nullptr; }
	virtual const UScriptStruct* GetInputType() const { return nullptr; }
	virtual void SetInputType(const UScriptStruct* Type) { }
	virtual void InputValueChanged() {};

	virtual bool HasPrimaryInput()
	{
		return true;
	}

	// random columns must go last, and get a special icon
	// using a virtual fucntion to identify them (rather than hard coding a specific type) to potentially support multiple varieties of randomization column.
	virtual bool IsRandomizeColumn() const { return false; }

	virtual bool AutoPopulates() const { return false; }
	virtual void AutoPopulate(int32 RowIndex, UObject* OutputObject) { }

	virtual float EditorTestCost(int32 RowIndex) const { return 0.0f; }
	virtual bool EditorTestFilter(int32 RowIndex) const { return false; }

	virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) {}
	virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) {}

	virtual void ReplaceReferences(UObject* ReferenceToReplace, UObject* ReplaceWith) {}
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Base)
	bool bDisabled = false;
	
	UPROPERTY()
	float EditorColumnWidth = 200;
#endif
};

enum {  ChooserColumn_SpecialIndex_Fallback = -2 };

#if WITH_EDITOR
#define CHOOSER_COLUMN_BOILERPLATE2(ParameterType, RowValuesProperty) \
	virtual FName RowValuesPropertyName() override { return #RowValuesProperty; }\
	virtual void SetNumRows(int32 NumRows) override\
	{\
		if (NumRows <= RowValuesProperty.Num())\
		{\
			RowValuesProperty.SetNum(NumRows);\
		}\
		else\
		{\
			while(RowValuesProperty.Num() < NumRows)\
			RowValuesProperty.Add(DefaultRowValue);\
		}\
	}\
	virtual void InsertRows(int Index, int Count) override\
	{\
		for (int i=0;i<Count;i++)\
		{\
			RowValuesProperty.Insert(DefaultRowValue, Index);\
		}\
	}\
	virtual void DeleteRows(TArrayView<int> RowIndices ) override\
	{\
		for(int32 Index : RowIndices)\
		{\
			if (RowValuesProperty.IsValidIndex(Index)) \
			{\
				RowValuesProperty.RemoveAt(Index);\
			}\
		}\
	}\
	virtual void MoveRow(int SourceRowIndex, int TargetRowIndex) override\
	{\
		auto RowData = RowValuesProperty[SourceRowIndex];\
    	RowValuesProperty.RemoveAt(SourceRowIndex);\
    	if (SourceRowIndex < TargetRowIndex) { TargetRowIndex--; }\
    	RowValuesProperty.Insert(RowData, TargetRowIndex);\
	}\
	virtual void CopyRow(FChooserColumnBase& SourceColumn, int SourceRowIndex, int TargetRowIndex) override\
	{\
		if (RowValuesProperty.IsValidIndex(TargetRowIndex) && static_cast<decltype(this)>(&SourceColumn)->RowValuesProperty.IsValidIndex(SourceRowIndex)) \
		{\
			RowValuesProperty[TargetRowIndex]=static_cast<decltype(this)>(&SourceColumn)->RowValuesProperty[SourceRowIndex];\
		}\
	}\
	virtual UScriptStruct* GetInputBaseType() const override { return ParameterType::StaticStruct(); };\
	virtual const UScriptStruct* GetInputType() const override { return InputValue.IsValid() ? InputValue.GetScriptStruct() : nullptr; };\
	virtual FChooserParameterBase* GetInputValue() override { return InputValue.GetMutablePtr<FChooserParameterBase>(); };\
	virtual FInstancedStruct* GetInputValuePtr() override { return &InputValue; };\
	virtual void SetInputType(const UScriptStruct* Type) override { InputValue.InitializeAs(Type); };

#else

#define CHOOSER_COLUMN_BOILERPLATE2(ParameterType, RowValuesProperty)\
	virtual FChooserParameterBase* GetInputValue() override { return InputValue.GetMutablePtr<FChooserParameterBase>(); };\

#endif

#define CHOOSER_COLUMN_BOILERPLATE(ParameterType) CHOOSER_COLUMN_BOILERPLATE2(ParameterType, RowValues)
