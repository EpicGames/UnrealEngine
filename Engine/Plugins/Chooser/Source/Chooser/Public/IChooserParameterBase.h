// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IHasContext.h"
#include "IObjectChooser.h"
#include "IChooserParameterBase.generated.h"

struct FAssetRegistryTag;

USTRUCT()
struct FChooserParameterBase
{
	GENERATED_BODY()

	virtual FString GetDebugName() const
	{
		FText Name;
		GetDisplayName(Name);
		return Name.ToString();
	}
	
	virtual void GetDisplayName(FText& OutName) const { }
	virtual void AddSearchNames(FStringBuilderBase& Builder) const { }
	virtual void ReplaceString(FStringView FindString, ESearchCase::Type, bool MatchWholeWord, FStringView ReplaceString) { }

	virtual void PostLoad() {};
	virtual void Compile(IHasContextClass* Owner, bool bForce) {};
	virtual bool HasCompileErrors(FText& OutMessage) const { return false; }

#if WITH_EDITOR
	virtual FName GetIconName() const
	{
		FText Unused;
		return HasCompileErrors(Unused) ? FName("Icons.WarningWithColor") : FName("Kismet.Tabs.Variables");
	}
	virtual FLinearColor GetIconColor() const
	{
		return FLinearColor(0.0f, 0.1f, 0.6f, 1.0f);
	}
#endif

	virtual ~FChooserParameterBase() = default;
};
