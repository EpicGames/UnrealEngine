// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "IChooserParameterBase.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserParameterBool.generated.h"

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UChooserParameterBool : public UInterface
{
	GENERATED_BODY()
};

class IChooserParameterBool
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const {}
};

USTRUCT()
struct FChooserParameterBoolBase : public FChooserParameterBase
{
	GENERATED_BODY()
    
    public:
    	virtual bool GetValue(FChooserEvaluationContext& Context, bool& OutResult) const { return false; }
    	virtual bool SetValue(FChooserEvaluationContext& Context, bool InValue) const { return false; }

#if WITH_EDITOR
		virtual FLinearColor GetIconColor() const override
    	{
    		return FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f);
    	}
#endif
};
