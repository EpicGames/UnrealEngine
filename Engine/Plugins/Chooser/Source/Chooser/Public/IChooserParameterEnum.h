// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserParameterBase.h"
#include "IChooserParameterEnum.generated.h"

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UChooserParameterEnum : public UInterface
{
	GENERATED_BODY()
};

class IChooserParameterEnum
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const {}
};

USTRUCT()
struct FChooserParameterEnumBase : public FChooserParameterBase
{
	GENERATED_BODY()
	
		virtual bool GetValue(FChooserEvaluationContext& Context, uint8& OutResult) const { return false; }
    	virtual bool SetValue(FChooserEvaluationContext& Context, uint8 InValue) const { return false; }

	#if WITH_EDITOR
    	virtual const UEnum* GetEnum() const { return nullptr; }

		virtual FLinearColor GetIconColor() const override
		{
			return FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f);
		}
    #endif
};
