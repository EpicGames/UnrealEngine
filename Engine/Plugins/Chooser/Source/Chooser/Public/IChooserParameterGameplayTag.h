// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserParameterBase.h"
#include "IChooserParameterGameplayTag.generated.h"

struct FGameplayTagContainer;
struct FGameplayTagQuery;

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UChooserParameterGameplayTag : public UInterface
{
	GENERATED_BODY()
};

class IChooserParameterGameplayTag
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const {}
};

USTRUCT()
struct FChooserParameterGameplayTagBase : public FChooserParameterBase
{
	GENERATED_BODY()
	virtual bool GetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer*& OutResult) const { return false; }
	virtual bool SetValue(FChooserEvaluationContext& Context, const FGameplayTagContainer& Value) const { return false; }
};

USTRUCT()
struct FChooserParameterGameplayTagQueryBase : public FChooserParameterBase
{
	GENERATED_BODY()
	virtual bool SetValue(FChooserEvaluationContext& Context, const FGameplayTagQuery& Value) const { return false; }
};
