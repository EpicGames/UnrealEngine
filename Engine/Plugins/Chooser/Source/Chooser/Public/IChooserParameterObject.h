// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterBase.h"
#include "IChooserParameterObject.generated.h"

class UObject;

USTRUCT()
struct FChooserParameterObjectBase : public FChooserParameterBase
{
	GENERATED_BODY()

	virtual bool GetValue(FChooserEvaluationContext& Context, FSoftObjectPath& OutResult) const { return false; }
	virtual bool GetValue(FChooserEvaluationContext& Context, UObject*& OutResult) const { return false; }
	virtual bool SetValue(FChooserEvaluationContext& Context, const FSoftObjectPath& OutResult) const { return false; }
	virtual bool SetValue(FChooserEvaluationContext& Context, UObject* OutResult) const { return false; }

#if WITH_EDITOR
	virtual UClass* GetAllowedClass() const { return UObject::StaticClass(); }

	virtual FLinearColor GetIconColor() const override
	{
		return FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);
	}
#endif
};
