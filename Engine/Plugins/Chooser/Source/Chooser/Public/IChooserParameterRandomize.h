// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"
#include "IChooserParameterBase.h"
#include "IChooserParameterRandomize.generated.h"

struct FRandomizationState
{
	// A ring buffer of recently selected rows (size determined by the size of FRandomizeColumn::RepeatProbabilityMultipliers)
	// Where each entry is a row index into the ChooserTable, which was recently selected.
	TArray<int32, TInlineAllocator<4>> RecentlySelectedRows;

	// Index into RecentlySelectedRows marking the one most recently selected entry (the next most recently selected entry is at (LastSelectedRowIndex - 1 + Num) % Num
	int LastSelectedRow;
};

USTRUCT(BlueprintType)
struct FChooserRandomizationContext
{
	GENERATED_BODY();

	TMap<const void*, FRandomizationState> StateMap;
};

USTRUCT()
struct FChooserParameterRandomizeBase : public FChooserParameterBase
{
	GENERATED_BODY()
	virtual bool GetValue(FChooserEvaluationContext& Context, FChooserRandomizationContext*& OutResult) const { return false; }
	virtual bool IsBound() const { return false; }
};
