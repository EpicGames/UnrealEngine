// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterBase.h"
#include "IChooserParameterStruct.generated.h"

struct FInstancedStruct;
class UStruct;

USTRUCT()
struct FChooserParameterStructBase : public FChooserParameterBase
{
	GENERATED_BODY()

	virtual bool SetValue(FChooserEvaluationContext& Context, const FInstancedStruct &Value)  const { return false; }

#if WITH_EDITOR
	virtual UScriptStruct* GetStructType() const { return nullptr; }
#endif
};
