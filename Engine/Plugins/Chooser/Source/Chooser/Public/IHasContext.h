// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"

#include "IHasContext.generated.h"

CHOOSER_API DECLARE_LOG_CATEGORY_EXTERN(LogChooser, Log, All);

UINTERFACE(MinimalAPI)
class UHasContextClass : public UInterface
{
	GENERATED_BODY()
};

DECLARE_MULTICAST_DELEGATE(FContextClassChanged)

class IHasContextClass
{
	GENERATED_BODY()
public:
	FContextClassChanged OnContextClassChanged;
	virtual TConstArrayView<FInstancedStruct> GetContextData() const { return TConstArrayView<FInstancedStruct>(); }

	virtual FString GetContextOwnerName() const { return ""; }
	virtual UObject* GetContextOwnerAsset() { return nullptr; }

	virtual void Compile(bool bForce = false) {}
#if WITH_EDITOR
	virtual void AddCompileDependency(const UStruct* Struct) { }
#endif

};

DECLARE_MULTICAST_DELEGATE_OneParam(FChooserOutputObjectTypeChanged, const UClass* OutputObjectType);

UENUM()
enum class EObjectChooserResultType
{
	// The Chooser returns an Object of the specified Result Class
	ObjectResult UMETA(DisplayName = "Object Of Type"),
	// The Chooser returns a Class that is a SubClass of the specified Result Class (eg a chooser could return a type of Character to spawn, or a type of AnimInstance to link)
	ClassResult UMETA(DisplayName = "SubClass Of"),
	// The Chooser returns nothing, but can write to one or more outputs (useful if you are only interested in returning integral types like a float or string)
	NoPrimaryResult UMETA(Hidden),
};
