// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "IObjectChooser.generated.h"

#if UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING && !UE_BUILD_TEST
#define CHOOSER_TRACE_ENABLED 1
#else
#define CHOOSER_TRACE_ENABLED 0
#endif

#define CHOOSER_DEBUGGING_ENABLED ((CHOOSER_TRACE_ENABLED) || (WITH_EDITOR))

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UObjectChooser : public UInterface
{
	GENERATED_BODY()
};

class IObjectChooser
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const { }
};

#if CHOOSER_DEBUGGING_ENABLED
struct FChooserDebuggingInfo
{
	const UObject* CurrentChooser = nullptr;
	bool bCurrentDebugTarget = false;
};
#endif


USTRUCT()
struct FChooserEvaluationInputObject
{
	GENERATED_BODY()
	FChooserEvaluationInputObject() {}
	FChooserEvaluationInputObject (UObject* InObject) : Object(InObject) {}
	TObjectPtr<UObject> Object;
};


// special context entry for mult-struct output keeps copies of a context struct for each returned result object
USTRUCT()
struct FChooserEvaluationOutputArray
{
	GENERATED_BODY()
	FChooserEvaluationOutputArray() {}
	FChooserEvaluationOutputArray(uint32 InSourceParamIndex)
	{
		SourceParamIndex = InSourceParamIndex;	
	}

	TArray<FInstancedStruct> OutputValues;
	uint32 SourceParamIndex = 0;
};

USTRUCT(BlueprintType)
struct FChooserEvaluationContext
{
	FChooserEvaluationContext() {}
	FChooserEvaluationContext(UObject* ContextObject)
	{
		AddObjectParam(ContextObject);
	}

	// Add a UObject Parameter to the context
	void AddObjectParam(UObject* Param)
	{
		ObjectParams.Add(Param);
		AddStructParam(ObjectParams.Last());
	}

	// Helper to get the first object parameter if there is one
	UObject* GetFirstObjectParam()
	{
		return ObjectParams.Num() > 0 ? ObjectParams[0].Object : nullptr;
	}
	
	// Helper to get the first object parameter if there is one
    const UObject* GetFirstObjectParam() const
    {
		return ObjectParams.Num() > 0 ? ObjectParams[0].Object : nullptr;
    }

	// Add a struct view Parameter to the Context
	// the struct will be referred to by reference, and so must have a lifetime that is longer than this context
	void AddStructViewParam(FStructView Param)
	{
		Params.Add(Param);
	}
	
	// Add a struct Parameter to the Context
	// the struct will be referred to by reference, and so must have a lifetime that is longer than this context
	template <class T>
	void AddStructParam(T& Param)
	{
		AddStructViewParam(FStructView::Make(Param));
	}

	#if CHOOSER_DEBUGGING_ENABLED
    	FChooserDebuggingInfo DebuggingInfo;
    #endif
	
	GENERATED_BODY()
	TArray<FStructView, TInlineAllocator<4>> Params;

	// storage for Object Params, call AddObjectParam to allocate one FChooserEvaluationInputObject in this array and then add a StructView of it to the Params array
	TArray<FChooserEvaluationInputObject, TInlineAllocator<4>> ObjectParams;
	
	// storage for output struct data in multi-output mode
	TArray<FChooserEvaluationOutputArray, TInlineAllocator<2>> OutputArrays;

	FRandomStream* RandomStream = nullptr;
};

USTRUCT()
struct FObjectChooserBase
{
	GENERATED_BODY()

public:
	virtual ~FObjectChooserBase() = default;

	enum class EIteratorStatus
	{
		Continue, // Succeeded, but continue on to the next entry (for getting multiple results from choosers)
		ContinueWithOutputs = Continue, // Backwards compatibility from when only single outputs were supported
		Stop,     // Succeeded, and stop iterating now  (for getting first result from choosers)
		Failed,   // Failed, continue iterating to the next entry  (will trigger chooser to continue evaluating to find a valid result, including evaluating the fallback row)
	};

	DECLARE_DELEGATE_RetVal_OneParam( EIteratorStatus, FObjectChooserIteratorCallback, UObject*);
	DECLARE_DELEGATE_RetVal_OneParam( EIteratorStatus, FObjectChooserSoftObjectIteratorCallback, const TSoftObjectPtr<UObject>&);

	virtual void Compile(class IHasContextClass* HasContext, bool bForce) {};
	virtual bool HasCompileErrors(FText& Message) { return false; }

	virtual void ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const
	{
		// fallback implementation
		Result = TSoftObjectPtr<UObject>(ChooseObject(Context));
	}
	
	virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const { return nullptr; };
	virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext& ContextData, FObjectChooserIteratorCallback Callback) const
	{
		// fallback implementation just calls the single version
		if (UObject* Result = ChooseObject(ContextData))
		{
			return Callback.Execute(Result);
		}
		return EIteratorStatus::Failed;
	}
	
	virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext& ContextData, FObjectChooserSoftObjectIteratorCallback Callback) const
	{
		// fallback implementation just calls the single version
		TSoftObjectPtr<UObject> Result;
		ChooseObject(ContextData, Result);
		if (!Result.IsNull())
		{
			return Callback.Execute(Result);
		}
	
		return EIteratorStatus::Failed;
	}


	virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const { return EIteratorStatus::Continue; }
	virtual void GetDebugName(FString& OutDebugName) const {};

#if WITH_EDITOR
	// for implementations with a reference to an asset or other object, get the object for use in editor
	virtual UObject* GetReferencedObject() const { return nullptr; };
#endif
};
