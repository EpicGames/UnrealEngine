// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EnumColumn.h"
#include "IChooserColumn.h"
#include "IChooserParameterEnum.h"
#include "StructUtils/InstancedStruct.h"
#include "ChooserPropertyAccess.h"
#include "Serialization/MemoryReader.h"
#include "MultiEnumColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT()
struct FChooserMultiEnumRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Runtime)
	uint32 Value = 0;

	bool Evaluate(const uint32 LeftHandSide) const
	{
		return Value == 0 || Value & LeftHandSide;
	}
};

USTRUCT(DisplayName = "Enum (Or)", Meta = (Category = "Filter", Tooltip = "A column which filters rows using an Enum variable, where rows pass if the enum is one of any of the checked values."))
struct FMultiEnumColumn : public FChooserColumnBase
{
	GENERATED_BODY()
public:
	UE_API FMultiEnumColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterEnumBase", ToolTip="The Enum property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserMultiEnumRowData DefaultRowValue;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserMultiEnumRowData> RowValues;

	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;
	
#if WITH_EDITOR
	const UEnum* GetEnum() const 
	{
		const UEnum* Enum = nullptr;
		if (const FChooserParameterEnumBase* Input = InputValue.GetPtr<FChooserParameterEnumBase>())
		{
			Enum = Input->GetEnum();
		}
		return Enum;
	}
	
	mutable uint8 TestValue = 0;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return RowValues.IsValidIndex(RowIndex) && RowValues[RowIndex].Evaluate(1 << TestValue);
	}
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		Reader << TestValue;
	}
	
	UE_API virtual void AddToDetails (FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex);
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex);
#endif
	
	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterEnumBase);

	UE_API virtual void PostLoad() override;
};

#undef UE_API
