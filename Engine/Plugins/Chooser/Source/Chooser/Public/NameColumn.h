// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "ChooserParameterName.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "NameColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT(DisplayName = "Name Binding")
struct FNameContextProperty : public FChooserParameterNameBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Meta = (BindingType = "FName", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;

	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, FName& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, const FName& InValue) const override;

	CHOOSER_PARAMETER_BOILERPLATE();
};

UENUM()
enum class ENameColumnCellValueComparison
{
	MatchEqual,
	MatchNotEqual,
	MatchAny,

	Modulus // used for cycling through the other values
};

USTRUCT()
struct FChooserNameRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Runtime, Meta = (ValidEnumValues = "MatchEqual, MatchNotEqual, MatchAny"))
	ENameColumnCellValueComparison Comparison = ENameColumnCellValueComparison::MatchEqual;

	UPROPERTY(EditAnywhere, Category = "Runtime")
	FName Value;

	bool Evaluate(const FName& LeftHandSide) const;
};

USTRUCT(DisplayName = "Name", Meta = (Category = "Filter", Tooltip = "A column which filters rows by an input Name compared to Names specified on each row."))
struct FChooserNameColumn : public FChooserColumnBase
{
	GENERATED_BODY()

	UE_API FChooserNameColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterNameBase", ToolTip="The Name reference property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserNameRowData DefaultRowValue;
#endif

	UPROPERTY()
	TArray<FChooserNameRowData> RowValues;

	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable FName TestValue;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return RowValues.IsValidIndex(RowIndex) && RowValues[RowIndex].Evaluate(TestValue);
	}
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		FString StringName;
		Reader << StringName;
		TestValue = FName(StringName);
	}

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterNameBase);

#if WITH_EDITOR
	virtual void PostLoad() override
	{
		Super::PostLoad();

		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}
#endif
};

#undef UE_API
