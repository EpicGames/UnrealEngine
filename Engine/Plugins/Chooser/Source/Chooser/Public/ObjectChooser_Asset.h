// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IObjectChooser.h"
#include "ObjectChooser_Asset.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Asset", Meta = (ResultType = "Object", Category = "Basic", Tooltip = "A hard reference to a specific asset"))
struct FAssetChooser : public FObjectChooserBase
{
	GENERATED_BODY()
	
	// FObjectChooserBase interface
	UE_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	UE_API virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const final override;

#if WITH_EDITOR
	virtual UObject* GetReferencedObject() const override { return Asset; }
#endif
	
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (GetAssetFilter="ResultAssetFilter"))
	TObjectPtr<UObject> Asset;
};

USTRUCT(DisplayName = "Asset (Soft Reference)", Meta = (ResultType = "Object", Category = "Basic", Tooltip = "A soft object reference to a specific asset\nAssets will need to be preloaded manually to avoid a hitch if they are selected."))
struct FSoftAssetChooser : public FObjectChooserBase
{
	GENERATED_BODY()

	// FObjectChooserBase interface
	UE_API virtual void ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const final override;
	UE_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	UE_API virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const final override;
	
#if WITH_EDITOR
	virtual UObject* GetReferencedObject() const override { return Asset.LoadSynchronous(); }
#endif
	
	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (GetAssetFilter="ResultAssetFilter"))
	TSoftObjectPtr<UObject> Asset;
};

// deprecated class for upgrading old data
UCLASS(MinimalAPI, ClassGroup = "LiveLink", deprecated)
class UDEPRECATED_ObjectChooser_Asset : public UObject, public IObjectChooser
{
	GENERATED_BODY()
	
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const override
	{
		OutInstancedStruct.InitializeAs(FAssetChooser::StaticStruct());
		FAssetChooser& AssetChooser = OutInstancedStruct.GetMutable<FAssetChooser>();
		AssetChooser.Asset = Asset;
	}
	
	UPROPERTY(EditAnywhere, Category = "Parameters")
	TObjectPtr<UObject> Asset;
};

#undef UE_API
