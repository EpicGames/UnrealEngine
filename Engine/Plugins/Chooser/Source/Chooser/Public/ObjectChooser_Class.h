// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IObjectChooser.h"
#include "ObjectChooser_Class.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Class", Meta = (ResultType = "Class", Category = "Basic", Tooltip = "A reference to a Class.\nOnly for use in Choosers with ResultType set to Sub Class Of"))
struct FClassChooser : public FObjectChooserBase
{
	GENERATED_BODY()
	
	// FObjectChooserBase interface
	UE_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	UE_API virtual EIteratorStatus IterateObjects(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const final override;
public: 
	UPROPERTY(EditAnywhere, Category = "Parameters")
	TObjectPtr<UClass> Class;
};

#undef UE_API
