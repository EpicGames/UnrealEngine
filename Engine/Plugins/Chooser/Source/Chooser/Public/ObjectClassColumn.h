// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterObject.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "ObjectColumn.h"
#include "ObjectClassColumn.generated.h"

#define UE_API CHOOSER_API

UENUM()
enum class EObjectClassColumnCellValueComparison
{
	Equal,
	NotEqual,
	SubClassOf,
	NotSubClassOf,
	Any,
};

USTRUCT()
struct FChooserObjectClassRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Runtime)
	EObjectClassColumnCellValueComparison Comparison = EObjectClassColumnCellValueComparison::SubClassOf;

	UPROPERTY(EditAnywhere, Category = "Runtime")
	TObjectPtr<UClass> Value;

	bool Evaluate(const UObject* LeftHandSide) const;
};

USTRUCT(DisplayName = "Object Class", Meta = (Category = "Filter", Tooltip = "A column which filters rows using an Object reference variable, by checking if that object is of a certain Class."))
struct FObjectClassColumn : public FChooserColumnBase
{
	GENERATED_BODY()

	UE_API FObjectClassColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterObjectBase", ToolTip="The Object reference property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Data", Meta =(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"))
	FChooserObjectClassRowData DefaultRowValue;
#endif

	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserObjectClassRowData> RowValues;

	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable FSoftObjectPath TestValue;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		if (RowValues.IsValidIndex(RowIndex))
		{
			if (UObject* Object = TestValue.ResolveObject())
			{
				return RowValues[RowIndex].Evaluate(Object);
			}
		}
		return false;
	}
	
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		FString Path;
		Reader << Path;
		TestValue.SetPath(Path);
	}

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterObjectBase);

#if WITH_EDITOR
	virtual void PostLoad() override
	{
		Super::PostLoad();

		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}
#endif
};

#undef UE_API
