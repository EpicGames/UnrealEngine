// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterObject.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "ObjectColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT(DisplayName = "Object Property Binding")
struct FObjectContextProperty : public FChooserParameterObjectBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Meta = (BindingType = "object", BindingColor = "ObjectPinTypeColor"), Category = "Binding")
	FChooserObjectPropertyBinding Binding;

	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, FSoftObjectPath& OutResult) const override;
	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, UObject*& OutResult) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, UObject* InValue) const override;
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, const FSoftObjectPath& InValue) const override;

	CHOOSER_PARAMETER_BOILERPLATE();

#if WITH_EDITOR
	virtual UClass* GetAllowedClass() const override { return Binding.AllowedClass; }
#endif
};

UENUM()
enum class EObjectColumnCellValueComparison
{
	MatchEqual,
	MatchNotEqual,
	MatchAny,

	Modulus // used for cycling through the other values
};

USTRUCT()
struct FChooserObjectRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Runtime, Meta = (ValidEnumValues = "MatchEqual, MatchNotEqual, MatchAny"))
	EObjectColumnCellValueComparison Comparison = EObjectColumnCellValueComparison::MatchEqual;

	UPROPERTY(EditAnywhere, Category = "Runtime")
	TSoftObjectPtr<UObject> Value;

	bool Evaluate(const FSoftObjectPath& LeftHandSide) const;
};

USTRUCT(DisplayName = "Object", Meta = (Category = "Filter", Tooltip = "A column which filters rows by an input Object to specified Objects for each row."))
struct FObjectColumn : public FChooserColumnBase
{
	GENERATED_BODY()

	UE_API FObjectColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterObjectBase", ToolTip="The Object reference property this column will filter based on"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserObjectRowData DefaultRowValue;
#endif

	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserObjectRowData> RowValues;

	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;

#if WITH_EDITOR
	mutable FSoftObjectPath TestValue;
	virtual bool EditorTestFilter(int32 RowIndex) const override
	{
		return RowValues.IsValidIndex(RowIndex) && RowValues[RowIndex].Evaluate(TestValue);
	}
	virtual void SetTestValue(TArrayView<const uint8> Value) override
	{
		FMemoryReaderView Reader(Value);
		FString Path;
		Reader << Path;
		TestValue.SetPath(Path);
	}

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
#endif

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterObjectBase);

#if WITH_EDITOR
	virtual void PostLoad() override
	{
		Super::PostLoad();

		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}
#endif
};

#undef UE_API
