// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterBool.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "OutputBoolColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Output Bool", Meta = (Category = "Output", Tooltip = "A column which writes a Bool value."))
struct FOutputBoolColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FOutputBoolColumn();
	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;
	
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterBoolBase", ToolTip="The Bool property this column will write to"), Category = "Hidden")
	FInstancedStruct InputValue;

#if WITH_EDITOR
	mutable bool TestValue=false;

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { bFallbackValue = static_cast<FOutputBoolColumn&>(SourceColumn).bFallbackValue; }
#endif
	
	bool& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? bFallbackValue : RowValues[Index];
	}
	
	bool GetValueForIndex(int32 Index) const
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? bFallbackValue : RowValues[Index];
	}
	
	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY()
	bool bFallbackValue = false;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	bool DefaultRowValue = false;
#endif
	
	UPROPERTY()
	TArray<bool> RowValues; 

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterBoolBase);
};

#undef UE_API
