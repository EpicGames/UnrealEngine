// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterEnum.h"
#include "StructUtils/InstancedStruct.h"
#include "ChooserPropertyAccess.h"
#include "EnumColumn.h"
#include "OutputEnumColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT()
struct FChooserOutputEnumRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Runtime)
	uint8 Value = 0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = EditorOnly)
	FName ValueName;
#endif
};


USTRUCT(DisplayName = "Output Enum", Meta = (Category = "Output", Tooltip = "A column which writes an Enum value."))
struct FOutputEnumColumn : public FEnumColumnBase
{
	GENERATED_BODY()
public:
	UE_API FOutputEnumColumn();
	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterEnumBase", ToolTip="The Enum property this column will write to"), Category = "Data")
	FInstancedStruct InputValue;

	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY()
	FChooserOutputEnumRowData FallbackValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserOutputEnumRowData DefaultRowValue;

	UE_API virtual void PostLoad() override;
#endif
	
	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserOutputEnumRowData> RowValues;
	
#if WITH_EDITOR
	const UEnum* GetEnum() const 
	{
		const UEnum* Enum = nullptr;
		if (const FChooserParameterEnumBase* Input = InputValue.GetPtr<FChooserParameterEnumBase>())
		{
			Enum = Input->GetEnum();
		}
		return Enum;
	}

	mutable uint8 TestValue;

	UE_API virtual void AddToDetails (FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FOutputEnumColumn&>(SourceColumn).FallbackValue; }
	
	UE_API virtual void EnumChanged(const UEnum* Enum) override;
#endif
	FChooserOutputEnumRowData& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	const FChooserOutputEnumRowData& GetValueForIndex(int32 Index) const
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	
	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterEnumBase);
};

#undef UE_API
