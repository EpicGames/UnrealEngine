// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterFloat.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "OutputFloatColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Output Float", Meta = (Category = "Output", Tooltip = "A column which writes a Float value."))
struct FOutputFloatColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FOutputFloatColumn();
	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;
	
	UPROPERTY(EditAnywhere, Meta = (ExcludeBaseStruct, NoClear, BaseStruct = "/Script/Chooser.ChooserParameterFloatBase", ToolTip="The Float property this column will write to"), Category = "Hidden")
	FInstancedStruct InputValue;

#if WITH_EDITOR
	mutable double TestValue=false;

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FOutputFloatColumn&>(SourceColumn).FallbackValue; }
#endif
	
	double& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}

	double GetValueForIndex(int32 Index) const
   	{
   		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
   	}

	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY()
	double FallbackValue = 0.0;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	double DefaultRowValue = 0.0;
#endif
	
	UPROPERTY()
	TArray<double> RowValues; 

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterFloatBase);
};

#undef UE_API
