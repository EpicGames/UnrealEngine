// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IChooserColumn.h"
#include "IChooserParameterGameplayTag.h"
#include "ChooserPropertyAccess.h"
#include "GameplayTagContainer.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "OutputGameplayTagQueryColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Gameplay Tags Query Property Binding")
struct FGameplayTagQueryContextProperty :  public FChooserParameterGameplayTagQueryBase
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, NoClear, Meta = (BindingType = "FGameplayTagQuery", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;
	
	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, const FGameplayTagQuery& Value) const override;
	
	CHOOSER_PARAMETER_BOILERPLATE();
};

USTRUCT(DisplayName = "Output Gameplay Tag Query", Meta = (Category = "Output", Tooltip = "A column which writes a Gameplay Tag Query value."))
struct FOutputGameplayTagQueryColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FOutputGameplayTagQueryColumn();

	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;
	
	UPROPERTY(EditAnywhere,  NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.FChooserParameterGameplayTagQueryBase", ToolTip="The Gameplay Tag Query property this column will write to"), Category = "Hidden")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FGameplayTagQuery DefaultRowValue;
#endif
	
	UPROPERTY()
	TArray<FGameplayTagQuery> RowValues;
	
#if WITH_EDITOR
	mutable FGameplayTagQuery TestValue;
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FOutputGameplayTagQueryColumn&>(SourceColumn).FallbackValue; }
#endif

	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY()
   	FGameplayTagQuery FallbackValue;

	FGameplayTagQuery& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	
	const FGameplayTagQuery& GetValueForIndex(int32 Index) const
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterGameplayTagQueryBase);
};

#undef UE_API
