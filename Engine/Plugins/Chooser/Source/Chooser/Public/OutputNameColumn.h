// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "ChooserParameterName.h"
#include "IChooserColumn.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "OutputNameColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT(DisplayName = "Output Name", Meta = (Category = "Output", Tooltip = "A column which writes a FName value."))
struct FChooserOutputNameColumn : public FChooserColumnBase
{
	GENERATED_BODY()

	UE_API FChooserOutputNameColumn();
	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterNameBase", ToolTip="The Name reference property this column will write to"), Category = "Hidden")
	FInstancedStruct InputValue;

	UPROPERTY()
	TArray<FName> RowValues;

#if WITH_EDITOR
	mutable FName TestValue = NAME_None;
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FChooserOutputNameColumn&>(SourceColumn).FallbackValue; }
#endif

	FName& GetValueForIndex(int32 Index)
	{
		if (Index == ChooserColumn_SpecialIndex_Fallback)
		{
			return FallbackValue;
		}
		else if (!RowValues.IsValidIndex(Index))
		{
			return FallbackValue;
		}

		return RowValues[Index];
	}

	FName GetValueForIndex(int32 Index) const
   	{
		if (Index == ChooserColumn_SpecialIndex_Fallback)
		{
			return FallbackValue;
		}
		else if (!RowValues.IsValidIndex(Index))
		{
			return FallbackValue;
		}

		return RowValues[Index];
   	}

	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY()
	FName FallbackValue = NAME_None;

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterNameBase);

#if WITH_EDITOR
	virtual void PostLoad() override
	{
		Super::PostLoad();

		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FName DefaultRowValue = NAME_None;
#endif

};

#undef UE_API
