// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChooserPropertyAccess.h"
#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterObject.h"
#include "StructUtils/InstancedStruct.h"
#include "Serialization/MemoryReader.h"
#include "OutputObjectColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT(DisplayName = "Output Object", Meta = (Category = "Output", Tooltip = "A column which writes an Object reference."))
struct FChooserOutputObjectRowData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Value", Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ObjectChooserBase"))
	FInstancedStruct Value;
};

USTRUCT(DisplayName = "Output Object", Meta = (Category = "Output", Tooltip = "A column which an Object Reference."))
struct FOutputObjectColumn : public FChooserColumnBase
{
	GENERATED_BODY()

	UE_API FOutputObjectColumn();

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterObjectBase"), Category = "Data")
	FInstancedStruct InputValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	FChooserOutputObjectRowData DefaultRowValue;
#endif

	UPROPERTY()
	// array of results (cells for this column for each row in the table)
	// should match the length of the Results array
	TArray<FChooserOutputObjectRowData> RowValues;

	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
    UPROPERTY()
	FChooserOutputObjectRowData FallbackValue;

	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;

#if WITH_EDITOR
	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FOutputObjectColumn&>(SourceColumn).FallbackValue; }
#endif
	
	FChooserOutputObjectRowData& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	
	const FChooserOutputObjectRowData& GetValueForIndex(int32 Index) const
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterObjectBase);

	UE_API virtual void Compile(IHasContextClass* Owner, bool bForce) override;

#if WITH_EDITORONLY_DATA
	virtual void PostLoad() override
	{
		Super::PostLoad();

		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}
#endif
};

#undef UE_API
