// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterStruct.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "OutputStructColumn.generated.h"

#define UE_API CHOOSER_API

struct FBindingChainElement;

USTRUCT(DisplayName = "Struct Property Binding")
struct FStructContextProperty : public FChooserParameterStructBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Meta = (BindingType = "struct", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserStructPropertyBinding Binding;

	UE_API virtual bool SetValue(FChooserEvaluationContext& Context, const FInstancedStruct &Value) const override;

	CHOOSER_PARAMETER_BOILERPLATE();

#if WITH_EDITOR
	virtual UScriptStruct* GetStructType() const override { return Binding.StructType; }
#endif
};

USTRUCT(DisplayName = "Output Struct", Meta = (Category = "Output", Tooltip = "A column which writes all elements of a Struct.\n The data for each row must be set in the details panel, and the table cells will display values which differ from the struct default."))
struct FOutputStructColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FOutputStructColumn();
	virtual bool HasFilters() const override { return false; }
	virtual bool HasOutputs() const override { return true; }
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;
	
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ChooserParameterStructBase", ToolTip="The Struct property this column wil write to"), Category = "Hidden")
	FInstancedStruct InputValue;

#if WITH_EDITOR
	virtual void InputValueChanged() override;
	UE_API void StructTypeChanged();
	mutable FInstancedStruct TestValue;

	UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	virtual void CopyFallback(FChooserColumnBase& SourceColumn) override { FallbackValue = static_cast<FOutputStructColumn&>(SourceColumn).FallbackValue; }
#endif
	
	FInstancedStruct& GetValueForIndex(int32 Index)
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	
	const FInstancedStruct& GetValueForIndex(int32 Index) const
	{
		return Index == ChooserColumn_SpecialIndex_Fallback ? FallbackValue : RowValues[Index];
	}
	
	// FallbackValue will be used as the output value if the all rows in the chooser fail, and the FallbackResult from the chooser is used.
	UPROPERTY();
   	FInstancedStruct FallbackValue;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Meta = (StructTypeConst, ToolTip="DefaultRowValue will be assigned to cells when new rows are created"), Category=Data);
	FInstancedStruct DefaultRowValue;
#endif
	
	UPROPERTY(EditAnywhere, Meta = (StructTypeConst), Category=Data);
	TArray<FInstancedStruct> RowValues; 

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterStructBase);
};

#undef UE_API
