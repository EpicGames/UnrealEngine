// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserColumn.h"
#include "IChooserParameterRandomize.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "RandomizeColumn.generated.h"

#define UE_API CHOOSER_API

USTRUCT(DisplayName = "Randomize Property Binding")
struct FRandomizeContextProperty :  public FChooserParameterRandomizeBase
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Meta = (BindingType = "FChooserRandomizationContext", BindingAllowFunctions = "false", BindingColor = "StructPinTypeColor"), Category = "Binding")
	FChooserPropertyBinding Binding;
	
	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, FChooserRandomizationContext*& OutResult) const override;
	virtual bool IsBound() const override
	{
		return Binding.IsBoundToRoot || !Binding.PropertyBindingChain.IsEmpty();
	}

	CHOOSER_PARAMETER_BOILERPLATE();
};

USTRUCT(DisplayName = "Randomize", Meta = (Category = "Random", Tooltip = "The Randomize column will randomly select between whatever values have passed all filters.\n The value specified in each cell is a probability weighting for the row.\n  A row with a value twice as likely as another will be twice as likely to be selected.\n Using the optional RandomizationContext binding, it can track the most recent selection, and reduce the probability of randomly picking the same entry twice"))
struct FRandomizeColumn : public FChooserColumnBase
{
	GENERATED_BODY()
	public:
	UE_API FRandomizeColumn();
	
	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.RandomizeContextProperty", ToolTip="Optional reference to a ChooserRandomizationContext struct. If bound, this is used to store the most recent selection (for each Choosers referencing it - you only need to create one variable per Character or context), for use with RepateProbabilityMultiplier to reduce the chance of selecting the same entry twice."), Category = "Data")
	FInstancedStruct InputValue;
	
	UPROPERTY(EditAnywhere, Category= "Data", meta=(ClampMin="0.0",Tooltip="Multiplies the weight of recently chosen results, where the number of entries in this array indicates how many recently selected choices to remember (A value of zero means to avoid repeating an entry unless it's the only choice). You must bind a RandomizationContext struct for this feature to work."));
	TArray<float> RepeatProbabilityMultipliers;

	UPROPERTY(EditAnywhere, Category= "Data", meta=(ClampMin="0.0",Tooltip="When columns with scoring are used, randomize will pick from among all rows that have a cost nearly equal to the minumum cost, using this threshold"));
	float EqualCostThreshold = 0.001f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Data, meta=(ToolTip="DefaultRowValue will be assigned to cells when new rows are created"));
	float DefaultRowValue = 1.0f;

	UPROPERTY()
	float RepeatProbabilityMultiplier = 1.0f;
#endif
	
	UPROPERTY()
	TArray<float> RowValues; 
	
	UE_API virtual void Filter(FChooserEvaluationContext& Context, const FChooserIndexArray& IndexListIn, FChooserIndexArray& IndexListOut) const override;
	UE_API virtual void SetOutputs(FChooserEvaluationContext& Context, int RowIndex) const override;
	
	virtual bool HasFilters() const override { return true; }
	virtual bool HasOutputs() const override { return true; }

	#if WITH_EDITOR
    	virtual bool EditorTestFilter(int32 RowIndex) const override
    	{
    		return true;
    	}

		UE_API virtual void AddToDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
		UE_API virtual void SetFromDetails(FInstancedPropertyBag& PropertyBag, int32 ColumnIndex, int32 RowIndex) override;
	
		virtual bool HasPrimaryInput() override
		{
			return false;
		}
    #endif

	virtual void PostLoad() override
	{
#if WITH_EDITORONLY_DATA
		if (RepeatProbabilityMultiplier < 1.0f && RepeatProbabilityMultipliers.Num() == 0)
		{
			RepeatProbabilityMultipliers.Add(RepeatProbabilityMultiplier);
			RepeatProbabilityMultiplier = 1.0f;
		}
#endif
		
		if (InputValue.IsValid())
		{
			InputValue.GetMutable<FChooserParameterBase>().PostLoad();
		}
	}

	virtual void Compile(IHasContextClass* Owner, bool bForce) override
	{
		if (FChooserParameterRandomizeBase* Input = InputValue.GetMutablePtr<FChooserParameterRandomizeBase>())
		{
			// binding on randomize columns is optional, so don't call compile unless it's bound, to avoid error messages
			if (Input->IsBound())
			{
				Input->Compile(Owner, bForce);
			}
		}
	};

	CHOOSER_COLUMN_BOILERPLATE(FChooserParameterRandomizeBase);

#if WITH_EDITOR
	virtual bool IsRandomizeColumn() const override { return true; }
#endif
};

#undef UE_API
