// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChooserEditor : ModuleRules
	{
		public ChooserEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[] {"Chooser"});
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"Engine",
					"AssetTools",
					"AssetDefinition",
					"UnrealEd",
					"EditorWidgets",
					"ToolWidgets",
					"ToolMenus",
					"SlateCore",
					"Slate",
					"PropertyEditor",
					"InputCore",
					"EditorStyle",
					"PropertyEditor",
					"BlueprintGraph",
					"GraphEditor",
					"GameplayTags",
					"GameplayTagsEditor",
					"KismetCompiler",
					"BlendStack",
					"TraceAnalysis",
					"TraceLog",
					"TraceServices",
					"TraceInsights",
					"RewindDebuggerInterface",
					"GameplayInsights",
					"Persona",
					"DeveloperSettings",
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}
