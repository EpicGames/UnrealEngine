// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ChooserSignature.h"
#include "Styling/SlateStyleRegistry.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "ChooserEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ChooserSignature)

#define LOCTEXT_NAMESPACE "UAssetDefinition_ChooserSignature" 

const FSlateBrush* UAssetDefinition_ChooserSignature::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ChooserEditor::FChooserEditorStyle::Get().GetBrush("ChooserEditor.ChooserTableIconLarge");
}

const FSlateBrush* UAssetDefinition_ChooserSignature::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ChooserEditor::FChooserEditorStyle::Get().GetBrush("ChooserEditor.ChooserTableIconSmall");
}

EAssetCommandResult UAssetDefinition_ChooserSignature::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	FSimpleAssetEditor::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Objects);

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
