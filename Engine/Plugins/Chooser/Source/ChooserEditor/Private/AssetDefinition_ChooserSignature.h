// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chooser.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ChooserSignature.generated.h"

UCLASS()
class UAssetDefinition_ChooserSignature : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition_ChooserSignature", "ChooserSignatureDisplayName", "Chooser Signature"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,128,64)); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UChooserSignature::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Animation),
				FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_ChooserSignatureSubMenu", "Data Table"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
