// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ChooserTable.h"
#include "ChooserTableEditor.h"
#include "Styling/SlateStyleRegistry.h"
#include "ChooserEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ChooserTable)

#define LOCTEXT_NAMESPACE "UAssetDefinition_ChooserTable" 

const FSlateBrush* UAssetDefinition_ChooserTable::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ChooserEditor::FChooserEditorStyle::Get().GetBrush("ChooserEditor.ChooserTableIconLarge");
}

const FSlateBrush* UAssetDefinition_ChooserTable::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ChooserEditor::FChooserEditorStyle::Get().GetBrush("ChooserEditor.ChooserTableIconSmall");
}

EAssetCommandResult UAssetDefinition_ChooserTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	
	for (UObject* Object : Objects)
	{
		UE::ChooserEditor::FChooserTableEditor::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
