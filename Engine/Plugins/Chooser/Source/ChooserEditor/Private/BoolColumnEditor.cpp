// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoolColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "BoolColumn.h"
#include "OutputBoolColumn.h"
#include "SPropertyAccessChainWidget.h"
#include "ObjectChooserWidgetFactories.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "GraphEditorSettings.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "BoolColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateBoolColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FBoolColumn* BoolColumn = static_cast<FBoolColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Bool Tooltip", "Bool: cells pass if the cell value is equal to the column input value, or the cell is set to \"Any\"");
		const FText ColumnName = LOCTEXT("Bool","Bool");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SHorizontalBox)
					 + SHorizontalBox::Slot().FillWidth(1)
					 + SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
					[
						SNew(SCheckBox).IsEnabled_Lambda([Chooser](){ return !Chooser->HasDebugTarget(); })
						.IsChecked_Lambda([BoolColumn]() { return BoolColumn->TestValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						.OnCheckStateChanged_Lambda([Chooser, BoolColumn](ECheckBoxState NewValue) { BoolColumn->TestValue = NewValue == ECheckBoxState::Checked; })
					]
					 + SHorizontalBox::Slot().FillWidth(1);
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	
	return
	SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().FillWidth(1)
		+ SHorizontalBox::Slot().MaxWidth(100).HAlign(HAlign_Center)
		[
			SNew(SButton).ButtonStyle(FAppStyle::Get(),"FlatButton").HAlign(HAlign_Center)
    				.Text_Lambda([Row, BoolColumn]()
    				{
    					if (!BoolColumn->RowValuesWithAny.IsValidIndex(Row))
    					{
    						return FText();
    					}
    					switch (BoolColumn->RowValuesWithAny[Row])
    					{
							case EBoolColumnCellValue::MatchAny: return LOCTEXT("Any","Any");
							case EBoolColumnCellValue::MatchTrue: return LOCTEXT("True","True");
							case EBoolColumnCellValue::MatchFalse: return LOCTEXT("False","False");
    					}
    					return FText();
    				})
    				.OnClicked_Lambda([Chooser, Row, BoolColumn]()
    				{
    					if (BoolColumn->RowValuesWithAny.IsValidIndex(Row))
    					{
    						const FScopedTransaction Transaction(LOCTEXT("Edit Bool Cell Data", "Edit Bool Cell Data"));
    						Chooser->Modify(true);
    						BoolColumn->RowValuesWithAny[Row] = static_cast<EBoolColumnCellValue>((static_cast<int>(BoolColumn->RowValuesWithAny[Row]) + 1) % 3);
    					}
    					return FReply::Handled();
    				})
    	]
		+ SHorizontalBox::Slot().FillWidth(1);
}
	
TSharedRef<SWidget> CreateOutputBoolColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FOutputBoolColumn* BoolColumn = static_cast<FOutputBoolColumn*>(Column);

	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Bool Tooltip", "Output Bool: Writes the value from the cell from the selected row to the bound property");
		const FText ColumnName = LOCTEXT("Output Bool","Output Bool");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SHorizontalBox)
					 + SHorizontalBox::Slot().FillWidth(1)
					 + SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
					[
						SNew(SCheckBox).IsEnabled_Lambda([Chooser](){ return !Chooser->HasDebugTarget(); })
						.IsChecked_Lambda([BoolColumn]() { return BoolColumn->TestValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					]
					 + SHorizontalBox::Slot().FillWidth(1);
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}
	else if (Row == ColumnWidget_SpecialIndex_Fallback) 
	{
		// create fallback value widget
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().FillWidth(1)
		+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([ BoolColumn]() { return BoolColumn->bFallbackValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([Chooser, BoolColumn](ECheckBoxState State)
			{
				const FScopedTransaction Transaction(LOCTEXT("Change Bool Value", "Change Bool Value"));
				Chooser->Modify(true);
				BoolColumn->bFallbackValue = State == ECheckBoxState::Checked;
			})
		]
		+ SHorizontalBox::Slot().FillWidth(1);
	}

	return
	SNew(SHorizontalBox)
	+ SHorizontalBox::Slot().FillWidth(1)
	+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
	[
		SNew (SCheckBox)
		.OnCheckStateChanged_Lambda([Chooser, BoolColumn,Row](ECheckBoxState State)
		{
			if (BoolColumn->RowValues.IsValidIndex(Row))
			{
				const FScopedTransaction Transaction(LOCTEXT("Change Bool Value", "Change Bool Value"));
				Chooser->Modify(true);
				BoolColumn->RowValues[Row] = (State == ECheckBoxState::Checked);
			}
		})
		.IsChecked_Lambda([BoolColumn, Row]()
		{
			const bool value = (BoolColumn->RowValues.IsValidIndex(Row)) ? BoolColumn->RowValues[Row] : false;
			return value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
	]
	+ SHorizontalBox::Slot().FillWidth(1);
}
	
TSharedRef<SWidget> CreateBoolPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FBoolContextProperty* ContextProperty = reinterpret_cast<FBoolContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(true).BindingColor("BooleanPinTypeColor").TypeFilter("bool")
	.PropertyBindingValue(&ContextProperty->Binding)
	.OnValueChanged(ValueChanged);
}
			
void CreateBoolPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FBoolContextProperty>("bool", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}
	
void RegisterBoolWidgets()
{
	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterBoolBase::StaticStruct(), CreateBoolPropertyMenus);
	
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FBoolContextProperty::StaticStruct(), CreateBoolPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FBoolColumn::StaticStruct(), CreateBoolColumnWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputBoolColumn::StaticStruct(), CreateOutputBoolColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
