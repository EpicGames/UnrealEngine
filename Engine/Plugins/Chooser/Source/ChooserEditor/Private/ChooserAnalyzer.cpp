// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"
#include "ChooserProvider.h"
#include "TraceServices/Utils.h"

FChooserAnalyzer::FChooserAnalyzer(TraceServices::IAnalysisSession& InSession, FChooserProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FChooserAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_ChooserEvaluation, "Chooser", "ChooserEvaluation");
	Builder.RouteEvent(RouteId_ChooserEvaluation2, "Chooser", "ChooserEvaluation2");
	Builder.RouteEvent(RouteId_ChooserValue, "Chooser", "ChooserValue");
}

bool FChooserAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FChooserAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_ChooserEvaluation:
		{
			uint64 ChooserId = EventData.GetValue<uint64>("ChooserId");
			uint64 OwnerId = EventData.GetValue<uint64>("OwnerId");
			int32 SelectedIndex = EventData.GetValue<int32>("SelectedIndex");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			Provider.AppendChooserEvaluation(ChooserId, OwnerId, { SelectedIndex }, RecordingTime);
			break;
		}
		case RouteId_ChooserEvaluation2:
		{
			uint64 ChooserId = EventData.GetValue<uint64>("ChooserId");
			uint64 OwnerId = EventData.GetValue<uint64>("OwnerId");
			TArrayView<const int32> SelectedIndices = EventData.GetArrayView<int32>("SelectedIndices");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			Provider.AppendChooserEvaluation(ChooserId, OwnerId, SelectedIndices, RecordingTime);
			break;
		}
		case RouteId_ChooserValue:
		{
			uint64 ChooserId = EventData.GetValue<uint64>("ChooserId");
			uint64 OwnerId = EventData.GetValue<uint64>("OwnerId");
			FString Key = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Key", Context);
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			TArrayView<const uint8> SerializedData = EventData.GetArrayView<uint8>("Value");
			Provider.AppendChooserValue(ChooserId, OwnerId, RecordingTime, FName(Key), SerializedData);
			break;
		}
	}

	return true;
}
