// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

class FChooserProvider;
namespace TraceServices { class IAnalysisSession; }

class FChooserAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FChooserAnalyzer(TraceServices::IAnalysisSession& InSession, FChooserProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_ChooserEvaluation,
		RouteId_ChooserEvaluation2,
		RouteId_ChooserValue,
	};

	TraceServices::IAnalysisSession& Session;
	FChooserProvider& Provider;
};
