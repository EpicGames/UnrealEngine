// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserColumnHeader.h"

#include "ChooserEditorStyle.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserTableEditor.h"
#include "Widgets/Images/SImage.h"
#include "GraphEditorSettings.h"
#include "ScopedTransaction.h"
#include "SPropertyAccessChainWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "UObject/FastReferenceCollector.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "ChooserColumnHeader"

namespace UE::ChooserEditor
{
	TSharedRef<SWidget> GetPropertyMenu(UChooserTable* Chooser, FChooserColumnBase* Column)
	{
		FMenuBuilder MenuBuilder(true, MakeShared<FUICommandList>());

		if (UStruct* ColumnInputType = Column->GetInputBaseType())
		{
			FObjectChooserWidgetFactories::CreateParametersMenu(ColumnInputType,Chooser, Chooser->GetContextOwner(), Column->GetInputValuePtr(), MenuBuilder,
				[Column](){
					Column->InputValueChanged();
				});
		}

		return MenuBuilder.MakeWidget();
	}
	

	TSharedRef<SWidget> MakeColumnHeaderWidget(UChooserTable* Chooser, FChooserColumnBase* Column, const FText& ColumnName,const FText& ColumnTooltip, const FSlateBrush* ColumnIcon, TSharedPtr<SWidget> DebugWidget, FChooserWidgetValueChanged ValueChanged)
	{
		TSharedPtr<SWidget> InputValueWidget = nullptr;
		
		if (Column->HasPrimaryInput())
		{
			InputValueWidget = SNew(SComboButton)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.HeightOverride(16.0f)
					[
						SNew(SImage)
						.Image_Lambda([Column]()
						{
							static FName PropertyIcon(TEXT("Kismet.Tabs.Variables"));
							
							FName IconName = PropertyIcon;
								
							if (FChooserParameterBase* InputValue = Column->GetInputValue())
							{
								IconName = InputValue->GetIconName();						
							}
								
							return FAppStyle::GetBrush(IconName);
						})
						.ColorAndOpacity_Lambda([Column]()
						{
							if (FChooserParameterBase* InputValue = Column->GetInputValue())
							{
								return InputValue->GetIconColor();
						 	}
							return FLinearColor::Gray;
						})
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text_Lambda([Column]()
						{
							if (FChooserParameterBase* InputValue = Column->GetInputValue())
							{
								FText DisplayName;
								InputValue->GetDisplayName(DisplayName);
								return DisplayName;
							}
							return LOCTEXT("Bind Input Value", "[None]");
						})
				]
			]
			.ToolTipText_Lambda([Column]()
			{
				bool bHasErrors = false;
				FText ErrorMessage;
					
				if (FChooserParameterBase* InputValue = Column->GetInputValue())
				{
					bHasErrors = InputValue->HasCompileErrors(ErrorMessage);
				}
					
				return bHasErrors ?  ErrorMessage : LOCTEXT("Select value to Bind", "Select value to Bind");
			})
			.OnGetMenuContent_Lambda([Chooser, Column]()
				{
					return GetPropertyMenu(Chooser, Column);
				});
		}
	
		TSharedRef<SVerticalBox> ColumnHeaderWidget =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SBorder)
					.ToolTipText(ColumnTooltip)
					.BorderBackgroundColor(FLinearColor(0,0,0,0))
					.Content()
					[
						SNew(SImage).Image(ColumnIcon)
					]
				]
				+ SHorizontalBox::Slot().VAlign(VAlign_Center).FillWidth(1)
				[
					SNew(STextBlock)
						.Margin(FMargin(5, 0, 0, 0))
						.Text(ColumnName)
						.ToolTipText(ColumnTooltip)
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("ToggleDisableTooltip","Toggle disable this column.  Disabled columns will not be evaluated and will be stripped from cooked data."))
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked_Lambda([Column, Chooser]()
					{
						FScopedTransaction Transaction(LOCTEXT("Toggle Disable Column", "Toggle Disable Column"));
						Chooser->Modify();
						Column->bDisabled = !Column->bDisabled;
						return FReply::Handled();
					})
					.Content()
					[
						SNew(SOverlay)
							+ SOverlay::Slot()
							[
								SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Visible"))
								.Visibility_Lambda([Column]() { return Column->bDisabled ? EVisibility::Hidden : EVisibility::Visible; })
							]
							+ SOverlay::Slot()
							[
								SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Hidden"))
								.Visibility_Lambda([Column]() { return Column->bDisabled ? EVisibility::Visible : EVisibility::Hidden; })
							]
					]
				]
			]
			+ SVerticalBox::Slot().AutoHeight()
			[
				InputValueWidget ? InputValueWidget.ToSharedRef() : SNullWidget::NullWidget
			];
		
		if (DebugWidget.IsValid())
		{
			ColumnHeaderWidget->AddSlot().AutoHeight()
			[
				SNew(SBorder)
					.BorderBackgroundColor(FLinearColor(0,0,0,0))
					.Padding(FMargin(0,5,0,0))
					.Content()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot().AutoWidth()
						[
							SNew(SBorder)
							.ToolTipText(LOCTEXT("DebugTooltip","Debug Value: This value either comes from the attached debug target, or is manually entered.  It is used to colorize the column cells based on how they would evaulate for the given input."))
							.BorderBackgroundColor(FLinearColor(0,0,0,0))
							.Content()
							[
								SNew(SImage).Image(FAppStyle::Get().GetBrush("Debug"))
							]
						]
						+ SHorizontalBox::Slot().FillWidth(1)
						[
							DebugWidget.ToSharedRef()
						]
					]
			];
		}

		return ColumnHeaderWidget;
	}
	
}

#undef LOCTEXT_NAMESPACE
