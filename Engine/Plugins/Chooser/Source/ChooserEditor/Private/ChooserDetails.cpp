// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserDetails.h"

#include "ChooserTableEditor.h"

#include "Chooser.h"
#include "ChooserEditorWidgets.h"
#include "ChooserPropertyAccess.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserDetails)

#define LOCTEXT_NAMESPACE "ChooserDetails"

bool UChooserRowDetails::ResultAssetFilter(const FAssetData& AssetData)
{
	if (Chooser)
	{
		return Chooser->ResultAssetFilter(AssetData);
	}
	return true;
}

namespace UE::ChooserEditor
{
	
void FChooserDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(!Objects.IsEmpty());

	UChooserTable* Chooser = Cast<UChooserTable>(Objects[0]);
	
	IDetailCategoryBuilder& HiddenCategory = DetailBuilder.EditCategory(TEXT("Hidden"));

	TArray<TSharedRef<IPropertyHandle>> HiddenProperties;
	HiddenCategory.GetDefaultProperties(HiddenProperties);
	for(TSharedRef<IPropertyHandle>& PropertyHandle :  HiddenProperties)
	{
		// these (Results and Columns arrays) need to be hidden when showing the root ChooserTable properties
		// but still need to be EditAnywhere so that the Properties exist for displaying when you select a row or column (eg by FChooserRowDetails below)
		PropertyHandle->MarkHiddenByCustomization();
	}
}

// Make the details panel show the values for the selected row, showing each column value
void FChooserRowDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(!Objects.IsEmpty());

	UChooserRowDetails* Row = Cast<UChooserRowDetails>(Objects[0]);
	UChooserTable* Chooser = Row->Chooser;
	
	TSharedPtr<IPropertyHandle> ChooserProperty = DetailBuilder.GetProperty("Chooser", Row->StaticClass());
	DetailBuilder.HideProperty(ChooserProperty);
}

// Make the details panel show the values for the selected row, showing each column value
void FChooserColumnDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	check(!Objects.IsEmpty());

	UChooserColumnDetails* Column = Cast<UChooserColumnDetails>(Objects[0]);
	UChooserTable* Chooser = Column->Chooser;
	
	if (Chooser->ColumnsStructs.IsValidIndex(Column->Column))
	{
		IDetailCategoryBuilder& PropertiesCategory = DetailBuilder.EditCategory("Column Properties");

		TSharedPtr<IPropertyHandle> ChooserProperty = DetailBuilder.GetProperty("Chooser", Column->StaticClass());
		DetailBuilder.HideProperty(ChooserProperty);
	
		TSharedPtr<IPropertyHandle> ColumnsArrayProperty = ChooserProperty->GetChildHandle("ColumnsStructs");
		TSharedPtr<IPropertyHandle> CurrentColumnProperty = ColumnsArrayProperty->AsArray()->GetElement(Column->Column);

		IDetailPropertyRow& NewColumnProperty = PropertiesCategory.AddProperty(CurrentColumnProperty);
		// hide array add button
		NewColumnProperty.ShowPropertyButtons(false);
		// removing the column type, and just showing the column instance struct properties
		NewColumnProperty.CustomWidget(true);
	}
}

}

#undef LOCTEXT_NAMESPACE
