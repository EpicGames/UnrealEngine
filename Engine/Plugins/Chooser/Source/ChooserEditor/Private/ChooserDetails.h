// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "IDetailCustomization.h"
#include "StructUtils/PropertyBag.h"
#include "ChooserDetails.generated.h"

class UChooserTable;

// Class used for chooser editor details customization
UCLASS()
class UChooserRowDetails : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION()
	bool ResultAssetFilter(const FAssetData& AssetData); 

	UPROPERTY(EditAnywhere, Category="Properties", meta=(FixedLayout, ShowOnlyInnerProperties))
    FInstancedPropertyBag Properties;
	
	UPROPERTY(EditAnywhere, Instanced, Category="Hidden")
	TObjectPtr<UChooserTable> Chooser;
	int Row;
};

namespace UE::ChooserEditor
{
	
	class FChooserDetails : public IDetailCustomization
	{
	public:
		FChooserDetails() {};
		virtual ~FChooserDetails() override = default;

		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShareable( new FChooserDetails() );
		}

		// IDetailCustomization interface
		virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	};
		
	class FChooserRowDetails : public IDetailCustomization
	{
	public:
		FChooserRowDetails() {};
		virtual ~FChooserRowDetails() override = default;

		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShareable( new FChooserRowDetails() );
		}

		// IDetailCustomization interface
		virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	};
	
	class FChooserColumnDetails : public IDetailCustomization
    {
    public:
    	FChooserColumnDetails() {};
    	virtual ~FChooserColumnDetails() override = default;
    
    	static TSharedRef<IDetailCustomization> MakeInstance()
    	{
    		return MakeShareable( new FChooserColumnDetails() );
    	}
    
    	// IDetailCustomization interface
    	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
    };

	
}
