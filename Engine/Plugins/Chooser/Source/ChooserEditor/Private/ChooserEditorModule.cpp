// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserEditorModule.h"

#include "AnimNode_ChooserPlayer.h"
#include "BoolColumnEditor.h"
#include "ChooserEditorStyle.h"
#include "ChooserPropertyAccess.h"
#include "ChooserTableEditor.h"
#include "ChooserTableEditorCommands.h"
#include "CurveOverrideCustomization.h"
#include "EnumColumnEditor.h"
#include "Features/IModularFeatures.h"
#include "FloatDistanceColumnEditor.h"
#include "FloatRangeColumnEditor.h"
#include "FrameTimeCustomization.h"
#include "GameplayTagColumnEditor.h"
#include "GameplayTagQueryColumnEditor.h"
#include "IAssetTools.h"
#include "MultiEnumColumnEditor.h"
#include "ObjectColumnEditor.h"
#include "NameColumnEditor.h"
#include "ObjectClassColumnEditor.h"
#include "ObjectClassColumnEditor.h"
#include "OutputEnumColumnEditor.h"
#include "OutputFloatColumnEditor.h"
#include "OutputNameColumnEditor.h"
#include "OutputObjectColumnEditor.h"
#include "OutputStructColumnEditor.h"
#include "PropertyAccessChainCustomization.h"
#include "PropertyEditorModule.h"
#include "RandomizeColumnEditor.h"
#include "ChooserTrack.h"
#include "Kismet2/EnumEditorUtils.h"
#include "EnumColumn.h"
#include "UObject/UObjectIterator.h"
#include "SChooserTableWidget.h"
#include "ChooserTableViewModel.h"

#define LOCTEXT_NAMESPACE "ChooserEditorModule"

namespace UE::ChooserEditor
{
	
FChoosersTrackCreator GChoosersTrackCreator;

void FEnumChangedListener::PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	// iterate over all loaded choosers
	for (TObjectIterator<UChooserTable> It; It; ++It)
	{
		UChooserTable* Chooser = *It;
		for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			if (FEnumColumnBase* EnumColumn = ColumnData.GetMutablePtr<FEnumColumnBase>())
			{
				EnumColumn->EnumChanged(Changed);
			}
		}
	}
}

TSharedPtr<IChooserTableViewModel> FChooserEditorModule::CreateChooserTableViewModel(UChooserTable* ChooserTable)
{
	return MakeShared<FChooserTableViewModel>(ChooserTable);
}

TSharedPtr<SWidget> FChooserEditorModule::CreateChooserTableView(TSharedPtr<IChooserTableViewModel> ViewModel, TSharedPtr<FUICommandList> CommandList)
{
	return SNew(SChooserTableWidget).ViewModel(ViewModel).Commands(CommandList);
}

	
void RegisterChooserToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* ToolBar;
	FName ParentName = NAME_None;
	if (ToolMenus->IsMenuRegistered(IChooserTableViewModel::ChooserToolbarName))
	{
		ToolBar = ToolMenus->ExtendMenu(IChooserTableViewModel::ChooserToolbarName);
	}
	else
	{
		ToolBar = UToolMenus::Get()->RegisterMenu(IChooserTableViewModel::ChooserToolbarName, ParentName, EMultiBoxType::SlimHorizontalToolBar);
	}

	const FChooserTableEditorCommands& Commands = FChooserTableEditorCommands::Get();
	FToolMenuSection& Section = ToolBar->AddSection("Chooser", TAttribute<FText>());
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		Commands.EditChooserSettings,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon("EditorStyle", "FullBlueprintEditor.EditGlobalOptions")));

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AutoPopulateAll));


	Section.AddDynamicEntry("DebuggingCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UChooserEditorToolMenuContext* Context = InSection.FindContext<UChooserEditorToolMenuContext>();

		if (Context)
		{
			if (TSharedPtr<UE::ChooserEditor::IChooserTableViewModel> ViewModel = Context->ViewModel.Pin())
			{
				InSection.AddEntry(FToolMenuEntry::InitComboButton( "SelectDebugTarget",
					FToolUIActionChoice(),
					FNewToolMenuDelegate::CreateLambda([ViewModel](UToolMenu* InToolMenu)
					{
						static_cast<FChooserTableViewModel*>(ViewModel.Get() )->MakeDebugTargetMenu(InToolMenu);
					}),
					TAttribute<FText>::CreateLambda([ViewModel]()
					{
						UChooserTable* Chooser = ViewModel->GetRootChooser();
						if (Chooser->HasDebugTarget())
						{
							return  FText::FromString(Chooser->GetDebugTargetName());
						}
						else
						{
							return Chooser->GetEnableDebugTesting() ? LOCTEXT("Manual Testing", "Manual Testing") : LOCTEXT("Debug Target", "Debug Target");
						}
					}),
					LOCTEXT("Debug Target Tooltip", "Select an object that has recently been the context object for this chooser to visualize the selection results")));
			}
		}
	}));

}
	
void FChooserEditorModule::StartupModule()
{
	FChooserEditorStyle::Initialize();
	
	FChooserTableEditor::RegisterWidgets();
	RegisterGameplayTagWidgets();
	RegisterGameplayTagQueryWidgets();
	RegisterFloatDistanceWidgets();
	RegisterFloatRangeWidgets();
	RegisterOutputFloatWidgets();
	RegisterBoolWidgets();
	RegisterEnumWidgets();
	RegisterMultiEnumWidgets();
	RegisterOutputEnumWidgets();
	RegisterNameWidgets();
	RegisterOutputNameWidgets();
	RegisterObjectWidgets();
	RegisterObjectClassWidgets();
	RegisterOutputObjectWidgets();
	RegisterStructWidgets();
	RegisterRandomizeWidgets();
	
	FChooserTableEditorCommands::Register();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	PropertyModule.RegisterCustomPropertyTypeLayout("FloatProperty", FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FFrameTimeCustomization>(); }), MakeShared<FFrameTimePropertyTypeIdentifier>());
	PropertyModule.RegisterCustomPropertyTypeLayout(FAnimCurveOverride::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FCurveOverrideCustomization>(); }));
	PropertyModule.RegisterCustomPropertyTypeLayout(FAnimCurveOverrideList::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FCurveOverrideListCustomization>(); }));
	PropertyModule.RegisterCustomPropertyTypeLayout(FChooserPropertyBinding::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FPropertyAccessChainCustomization>(); }));
	PropertyModule.RegisterCustomPropertyTypeLayout(FChooserEnumPropertyBinding::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FPropertyAccessChainCustomization>(); }));
	PropertyModule.RegisterCustomPropertyTypeLayout(FChooserObjectPropertyBinding::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FPropertyAccessChainCustomization>(); }));
	PropertyModule.RegisterCustomPropertyTypeLayout(FChooserStructPropertyBinding::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FPropertyAccessChainCustomization>(); }));

	IModularFeatures::Get().RegisterModularFeature( IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerChooser);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GChoosersTrackCreator);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &ChooserTraceModule);

	FEnumEditorUtils::FEnumEditorManager::Get().AddListener(&EnumChanged);
	
	RegisterChooserToolbar();
}

void FChooserEditorModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature( IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerChooser);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GChoosersTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &ChooserTraceModule);
	FChooserTableEditorCommands::Unregister();
	
	FChooserEditorStyle::Shutdown();
	
	FEnumEditorUtils::FEnumEditorManager::Get().RemoveListener(&EnumChanged);
}

}

IMPLEMENT_MODULE(UE::ChooserEditor::FChooserEditorModule, ChooserEditor);

#undef LOCTEXT_NAMESPACE