// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChooserTraceModule.h"
#include "Modules/ModuleInterface.h"
#include "RewindDebuggerChooser.h"
#include "Kismet2/EnumEditorUtils.h"
#include "IChooserEditorModule.h"


class FUICommandList;

namespace UE::ChooserEditor
{

class FEnumChangedListener : public FEnumEditorUtils::FEnumEditorManager::BaseNotifyOnChanged
{
	virtual void PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;
	virtual void PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override {};
};

class FChooserEditorModule : public IChooserEditorModule
{
public: 
	virtual TSharedPtr<IChooserTableViewModel> CreateChooserTableViewModel(UChooserTable* ChooserTable) override;
	virtual TSharedPtr<SWidget> CreateChooserTableView(TSharedPtr<IChooserTableViewModel> ViewModel, TSharedPtr<FUICommandList> CommandList) override;

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FRewindDebuggerChooser RewindDebuggerChooser;
	FChooserTraceModule ChooserTraceModule;

	FEnumChangedListener EnumChanged;
};

}
