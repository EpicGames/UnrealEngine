// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserEditorSettings.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserEditorSettings)

UChooserEditorSettings::UChooserEditorSettings()
{
	FCoreDelegates::OnPreExit.AddLambda([]()
	{
		Get().SaveConfig();
	});
}

#if WITH_EDITOR

FText UChooserEditorSettings::GetSectionText() const
{
	return NSLOCTEXT("Chooser", "ChooserEditorSettingsName", "Chooser Editor");
}

FText UChooserEditorSettings::GetSectionDescription() const
{
	return NSLOCTEXT("Chooser", "ChooserEditorSettingsDesc", "Configure options for the Chooser Plugin.");
}

#endif

FName UChooserEditorSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

UChooserEditorSettings& UChooserEditorSettings::Get()
{
	UChooserEditorSettings* MutableCDO = GetMutableDefault<UChooserEditorSettings>();
	check(MutableCDO != nullptr)
	
	return *MutableCDO;
}

