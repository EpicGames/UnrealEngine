// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Logging/LogVerbosity.h"

#include "ChooserEditorSettings.generated.h"

UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="Chooser Editor"))
class UChooserEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UChooserEditorSettings();

#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif

	virtual FName GetCategoryName() const override;
	
	/** Most Recently Used Chooser Initializer, will be used as the default on following chooser table creates */
	UPROPERTY(Config)
	FString DefaultCreateType = "";

	/** Get Mutable CDO of UChooserEditorSettings */
	static UChooserEditorSettings& Get();
};
