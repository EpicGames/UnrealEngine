// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserEditorStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::ChooserEditor
{
	
TSharedPtr< FChooserEditorStyle > FChooserEditorStyle::StyleInstance = nullptr;


FChooserEditorStyle::FChooserEditorStyle() :
	FSlateStyleSet("ChooserEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Chooser/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT(""));

	// content browser icon
	Set("ChooserEditor.ChooserTableIconLarge", new IMAGE_BRUSH_SVG("ChooserIcon_24", Icon24x24));

	// tab icon
	Set("ChooserEditor.ChooserTableIconSmall", new IMAGE_BRUSH_SVG("Chooser", Icon16x16));
	
	Set("ChooserEditor.FallbackIcon", new IMAGE_BRUSH_SVG("Fallback", Icon16x16));
	
	// content browser icon
	Set("ChooserEditor.NestedChooserIcon", new IMAGE_BRUSH_SVG("NestedChooser", Icon16x16));
}

void FChooserEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = MakeShared<FChooserEditorStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FChooserEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

const ISlateStyle& FChooserEditorStyle::Get()
{
	return *StyleInstance;
}

}