// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserEditorWidgets.h"

#include "AssetTypeCategories.h"
#include "Chooser.h"
#include "ChooserEditorStyle.h"
#include "ChooserTableEditor.h"
#include "DetailLayoutBuilder.h"
#include "ObjectChooserClassFilter.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectChooser_Asset.h"
#include "ObjectChooser_Class.h"
#include "PropertyCustomizationHelpers.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/STextEntryPopup.h"

#define LOCTEXT_NAMESPACE "ChooserEditorWidgets"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateAssetWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	FAssetChooser* DIAsset = static_cast<FAssetChooser*>(Value);

	UObject* Asset = DIAsset->Asset;
	
	FAssetReferenceFilterContext AssetReferenceFilterContext;
	AssetReferenceFilterContext.AddReferencingAssets({&TransactionObject, 1});
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;	
	
	
	return SNew(SObjectPropertyEntryBox)
		.IsEnabled(!bReadOnly)
		.AllowedClass(ResultBaseClass ? ResultBaseClass : UObject::StaticClass())
		.ObjectPath_Lambda([DIAsset](){ return DIAsset->Asset ? DIAsset->Asset.GetPath() : "";})
		.bOnlyRecognizeOnDragEnter(true)
		.OnObjectChanged_Lambda([TransactionObject, DIAsset, ValueChanged](const FAssetData& AssetData)
		{
			const FScopedTransaction Transaction(LOCTEXT("Edit Asset", "Edit Asset"));
			TransactionObject->Modify(true);
			DIAsset->Asset = AssetData.GetAsset();
			ValueChanged.ExecuteIfBound();
		})
		.OnShouldFilterAsset_Lambda([AssetReferenceFilter](const FAssetData& AssetData)
		{
			if (AssetReferenceFilter.IsValid())
			{
				return !AssetReferenceFilter->PassesFilter(AssetData);
			}
			return false;
		});
}
	
TSharedRef<SWidget> CreateSoftAssetWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	FSoftAssetChooser* DIAsset = static_cast<FSoftAssetChooser*>(Value);

	FAssetReferenceFilterContext AssetReferenceFilterContext;
	AssetReferenceFilterContext.AddReferencingAssets({&TransactionObject, 1});
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
	
	return SNew(SObjectPropertyEntryBox)
		.IsEnabled(!bReadOnly)
		.AllowedClass(ResultBaseClass ? ResultBaseClass : UObject::StaticClass())
		.ObjectPath_Lambda([DIAsset](){ return DIAsset->Asset.ToSoftObjectPath().ToString(); })
		.OnObjectChanged_Lambda([TransactionObject, DIAsset, ValueChanged](const FAssetData& AssetData)
		{
			const FScopedTransaction Transaction(LOCTEXT("Edit Asset", "Edit Asset"));
			TransactionObject->Modify(true);
			DIAsset->Asset = AssetData.GetAsset();
			ValueChanged.ExecuteIfBound();
		})
		.OnShouldFilterAsset_Lambda([AssetReferenceFilter](const FAssetData& AssetData)
		{
			if (AssetReferenceFilter.IsValid())
			{
				return !AssetReferenceFilter->PassesFilter(AssetData);
			}
			return false;
		});
}

TSharedRef<SWidget> CreateClassWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	FClassChooser* ClassChooser = static_cast<FClassChooser*>(Value);

	UClass* Class = ClassChooser->Class;
	

	return SNew(SClassPropertyEntryBox)
			.IsEnabled(!bReadOnly)
			.MetaClass(ResultBaseClass ? ResultBaseClass : UObject::StaticClass())
			.SelectedClass_Lambda([ClassChooser]()
			{
				return ClassChooser->Class;
			})
			.OnSetClass_Lambda([TransactionObject, ClassChooser](const UClass* SelectedClass)
			{
				const FScopedTransaction Transaction(LOCTEXT("Edit Class", "Edit Class"));
				TransactionObject->Modify(true);
				ClassChooser->Class = const_cast<UClass*>(SelectedClass);
			});
}

TSharedRef<SWidget> CreateEvaluateChooserWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	FEvaluateChooser* EvaluateChooser = static_cast<FEvaluateChooser*>(Value);

	FAssetReferenceFilterContext AssetReferenceFilterContext;
	AssetReferenceFilterContext.AddReferencingAssets({&TransactionObject, 1});
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
	
	return SNew(SObjectPropertyEntryBox)
		.IsEnabled(!bReadOnly)
		.AllowedClass(UChooserTable::StaticClass())
		.bOnlyRecognizeOnDragEnter(true)
		.ObjectPath_Lambda([EvaluateChooser](){ return EvaluateChooser->Chooser ? EvaluateChooser->Chooser.GetPath() : "";})
		.OnShouldFilterAsset_Lambda([ResultBaseClass, AssetReferenceFilter](const FAssetData& InAssetData)
		{
			if (AssetReferenceFilter.IsValid())
			{
				if (!AssetReferenceFilter->PassesFilter(InAssetData))
				{
					// filter for assets outside the current plugin scope etc.
					return true;
				}
			}
				
			if (ResultBaseClass == nullptr)
		 	{
		 		return false;
		 	}
		 	if (InAssetData.IsInstanceOf(UChooserTable::StaticClass()))
		 	{
		 		if (UChooserTable* Chooser = Cast<UChooserTable>(InAssetData.GetAsset()))
		 		{
		 			UChooserTable* ContextOwner = Chooser->GetRootChooser();
		 			return !(ContextOwner->OutputObjectType && ContextOwner->OutputObjectType->IsChildOf(ResultBaseClass));
		 		}
		 	}
			return true;
		})
		.OnObjectChanged_Lambda([TransactionObject, EvaluateChooser, ValueChanged](const FAssetData& AssetData)
		{
			const FScopedTransaction Transaction(LOCTEXT("Edit Chooser", "Edit Chooser"));
			TransactionObject->Modify(true);
			EvaluateChooser->Chooser = Cast<UChooserTable>(AssetData.GetAsset());
			ValueChanged.ExecuteIfBound();
		});
}

TSharedRef<SWidget> CreateNestedChooserWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged, IChooserTableWidgetInterface* ChooserWidgetInterface)
{
	FNestedChooser* NestedChooser = static_cast<FNestedChooser*>(Value);
	
	TSharedRef<SComboButton> Button = SNew(SComboButton)
			.ContentPadding(0)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SImage).Image(FChooserEditorStyle::Get().GetBrush("ChooserEditor.NestedChooserIcon"))
				]
				+ SHorizontalBox::Slot().FillWidth(1.0).Padding(2)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Lambda([NestedChooser]()
					{
						if (NestedChooser->Chooser)
						{
							return FText::FromString(NestedChooser->Chooser->GetName());
						}
						else
						{
							return LOCTEXT("None", "None");
						}
					})
				]
			];
			
	Button->SetOnGetMenuContent(
		FOnGetContent::CreateLambda(
			[ Button, NestedChooser, TransactionObject]()
            			{
							FMenuBuilder MenuBuilder(true, nullptr);
                        
                        	MenuBuilder.BeginSection(NAME_None, LOCTEXT("Nested Chooser", "Nested Chooser"));
                        	{
							MenuBuilder.AddMenuEntry(
								LOCTEXT("None", "None"), 
								LOCTEXT("NewNestedChooser_Tooltip", "Create a new Chooser Table embedded in this asset, and reference it"),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateLambda([Button, NestedChooser, TransactionObject]()
								{
									const FScopedTransaction Transaction(LOCTEXT("Clear Nested Chooser", "Clear Nested Chooser"));
									TransactionObject->Modify(true);
									Button->SetIsOpen(false);
									NestedChooser->Chooser = nullptr;
								})));
								
                        		MenuBuilder.AddMenuEntry(
                        			LOCTEXT("New Nested Chooser", "New Nested Chooser"), 
                        			LOCTEXT("NewNestedChooser_Tooltip", "Create a new Chooser Table embedded in this asset, and reference it"),
                        			FSlateIcon(),
                        			FUIAction(FExecuteAction::CreateLambda([Button, NestedChooser, TransactionObject]()
                        			{
                        				Button->SetIsOpen(false);
                        				
										TSharedRef<STextEntryPopup> TextEntry =
											SNew(STextEntryPopup)
											.Label(LOCTEXT("NewNestedChooserLabel", "New Chooser Name"))
											.OnTextCommitted_Lambda([NestedChooser, TransactionObject](FText InText, ETextCommit::Type InCommitType)
											{
												if (InCommitType == ETextCommit::OnEnter)
												{
													FSlateApplication::Get().DismissAllMenus();
													FName NewChooserName = FName(InText.ToString());
													UChooserTable* ParentChooser = Cast<UChooserTable>(TransactionObject);
													UChooserTable* RootChooser = nullptr;

													if (NewChooserName == NAME_None || FindObject<UObject>(TransactionObject, NewChooserName.ToString()))
													{
														NewChooserName = MakeUniqueObjectName(TransactionObject, UChooserTable::StaticClass(), NewChooserName);
													}

													UChooserTable* NewChooser = NewObject<UChooserTable>(TransactionObject, UChooserTable::StaticClass(), NewChooserName, RF_Transactional);
												
													const FScopedTransaction Transaction(LOCTEXT("Assign New Nested Chooser", "Assign New Nested Chooser"));
												
													TransactionObject->Modify();
													NestedChooser->Chooser = NewChooser;
													
													if (ParentChooser)
													{
														RootChooser = ParentChooser->GetRootChooser();
													}

													if (RootChooser)
													{
														NestedChooser->Chooser->RootChooser = RootChooser;
														RootChooser->Modify();
														RootChooser->AddNestedObject(NestedChooser->Chooser);

														if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
														{
															// this causes the UAF workspace outliner to refresh
															AssetRegistry->AssetUpdateTags(RootChooser, EAssetRegistryTagsCaller::Fast);
														}
													}
												}
											});
							
										FSlateApplication& SlateApp = FSlateApplication::Get();
										SlateApp.PushMenu(
											SlateApp.GetInteractiveTopLevelWindows()[0],
											FWidgetPath(),
											TextEntry,
											SlateApp.GetCursorPos(),
											FPopupTransitionEffect::TypeInPopup
											);
                        			})));
								
								MenuBuilder.AddSubMenu(LOCTEXT("Select Existing", "Select Existing"),
									LOCTEXT("SelectExisting_Tooltip", "Select Existing Embedded UChooserTable from this asset"),
									FNewMenuDelegate::CreateLambda([Button, NestedChooser, TransactionObject](FMenuBuilder& SubMenuBuilder)
									{
										SubMenuBuilder.BeginSection("Existing", LOCTEXT("Existing Choosers", "Existing Choosers"));

										UChooserTable* OuterChooser = Cast<UChooserTable>(TransactionObject);
										UChooserTable* RootTable = nullptr;
										if (OuterChooser)
										{
											RootTable = OuterChooser->GetRootChooser();

											for (UObject* Object : RootTable->NestedObjects)
											{
												if (UChooserTable* Chooser = Cast<UChooserTable>(Object))
												{
													// exclude the main chooser 
												   if (Chooser != RootTable)
												   {
													   SubMenuBuilder.AddMenuEntry( FText::FromString(Chooser->GetName()), LOCTEXT("AddExistingObjectTooltip", "Add a reference to this existing Chooser Table."), FSlateIcon(),
														   FUIAction(FExecuteAction::CreateLambda([Chooser, Button, NestedChooser, TransactionObject]()
														   {
															   const FScopedTransaction Transaction(LOCTEXT("Set Nested Chooser", "Set Nested Chooser"));
															   TransactionObject->Modify(true);
															   Button->SetIsOpen(false);
															   NestedChooser->Chooser = Chooser;
														   })));
												   }
												}
											}
										}
									}));
                                     
									
									
                        	}

							return MenuBuilder.MakeWidget();

            			})
		);
		
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().FillWidth(1.0)
		[
			Button
		]
		+ SHorizontalBox::Slot().AutoWidth()
		[
			SNew(SButton)
			.Text(LOCTEXT("Edit","Edit"))
			.OnClicked_Lambda([NestedChooser, TransactionObject, ChooserWidgetInterface]()
			{
				if (NestedChooser->Chooser)
				{
					if (ChooserWidgetInterface)
					{
						ChooserWidgetInterface->OpenObject(NestedChooser->Chooser);
					}
					else
					{
						UObject* RootChooser = TransactionObject->GetPackage()->FindAssetInPackage();
						IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(RootChooser, false);
						if (Editor)
						{
							Editor->FocusWindow(NestedChooser->Chooser);
						}
					}
				}
				return FReply::Handled();
			})
		];
}
	
}

#undef LOCTEXT_NAMESPACE
