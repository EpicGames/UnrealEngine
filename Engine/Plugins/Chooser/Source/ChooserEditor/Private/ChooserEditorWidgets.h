// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ObjectChooserWidgetFactories.h"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateAssetWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged);
TSharedRef<SWidget> CreateSoftAssetWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged);
TSharedRef<SWidget> CreateClassWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged);
TSharedRef<SWidget> CreateEvaluateChooserWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged);
TSharedRef<SWidget> CreateNestedChooserWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged, IChooserTableWidgetInterface* ChooserWidgetInterface);
	
}