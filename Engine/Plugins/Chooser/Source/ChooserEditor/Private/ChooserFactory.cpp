// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserFactory.h"
#include "Chooser.h"
#include "ChooserInitializer.h"
#include "ChooserEditorSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SWindow.h"
#include "Editor.h"
#include "ObjectChooserClassFilter.h"
#include "StructViewerModule.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "SChooserCreateDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserFactory)

UChooserTableFactory::UChooserTableFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UChooserTable::StaticClass();
}

bool UChooserTableFactory::ConfigureProperties()
{
	TSharedRef<SChooserCreateDialog> Dialog = SNew(SChooserCreateDialog);
	return Dialog->ConfigureProperties(this);
}

UObject* UChooserTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	UChooserTable* NewChooser = nullptr;
	if (ChooserInitializer.IsValid())
	{
		const FChooserInitializer& Initializer = ChooserInitializer.Get<FChooserInitializer>();
		UClass* ChooserType = Initializer.OverrideClass(Class);
		NewChooser = NewObject<UChooserTable>(InParent, ChooserType, Name, Flags);
		Initializer.InitializeSignature(NewChooser);
		UChooserEditorSettings::Get().DefaultCreateType = ChooserInitializer.GetScriptStruct()->GetStructPathName().ToString();
	}
	else
	{
		NewChooser = NewObject<UChooserTable>(InParent, Class, Name, Flags);
	}
	
	NewChooser->Version = UChooserTable::CurrentVersion;
	
	return NewChooser;
}