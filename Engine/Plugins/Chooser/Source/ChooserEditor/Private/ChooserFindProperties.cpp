// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserFindProperties.h"

#include "Chooser.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserFindProperties)

#define LOCTEXT_NAMESPACE "ChooserFindProperties"

FString UChooserFindProperties::GetFindResultStringFromAssetData(const FAssetData& InAssetData) const
{
	if(GetFindString().IsEmpty())
	{
		return FString();
	}

	auto GetMatchingPropertyNamesForAsset = [this](const FAssetData& InAssetData, TArray<FString>& OutPropertyNames)
	{
		const FString TagValue = InAssetData.GetTagValueRef<FString>(UChooserTable::PropertyNamesTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *UChooserTable::PropertyTagDelimiter, [this, &OutPropertyNames](FStringView InToken)
			{
				if(GetFindWholeWord())
				{
					if(InToken.Compare(GetFindString(), GetSearchCase()) == 0)
					{
						OutPropertyNames.Add(FString(InToken));
					}
				}
				else
				{
					if(UE::String::FindFirst(InToken, GetFindString(), GetSearchCase()) != INDEX_NONE)
					{
						OutPropertyNames.Add(FString(InToken));
					}
				}
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	};

	TStringBuilder<128> Builder;
	TArray<FString> PropertyNames;
	GetMatchingPropertyNamesForAsset(InAssetData, PropertyNames);
	if(PropertyNames.Num() > 0)
	{
		for(int32 NameIndex = 0; NameIndex < PropertyNames.Num(); ++NameIndex)
		{
			Builder.Append(PropertyNames[NameIndex]);
			if(NameIndex != PropertyNames.Num() - 1)
			{
				Builder.Append(TEXT(", "));
			}
		}
	}
	return FString(Builder.ToString());
}

TConstArrayView<UClass*> UChooserFindProperties::GetSupportedAssetTypes() const
{
	static UClass* Types[] = { UChooserTable::StaticClass() };
	return Types;
}

bool UChooserFindProperties::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	FString TagValue;
	if(InAssetData.GetTagValue<FString>(UChooserTable::PropertyNamesTag, TagValue))
	{
		bOutIsOldAsset = false;

		if(GetFindString().IsEmpty())
		{
			return true;
		}
		
		bool bFoundMatch = false;
		UE::String::ParseTokens(TagValue, *UChooserTable::PropertyTagDelimiter, [this, &bFoundMatch](FStringView InToken)
		{
			if(NameMatches(InToken))
			{
				bFoundMatch = true;
			}
		}, UE::String::EParseTokensOptions::SkipEmpty);

		return !bFoundMatch;
	}
	
	bOutIsOldAsset = true;
	return true;
}

void UChooserFindProperties::ReplaceInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UChooserTable* ChooserTable = Cast<UChooserTable>(Asset))
		{
			Asset->MarkPackageDirty();

			for(FInstancedStruct& Column : ChooserTable->ColumnsStructs)
			{
				if (FChooserColumnBase* ColumnData = Column.GetMutablePtr<FChooserColumnBase>())
				{
					ColumnData->GetInputValue()->ReplaceString(GetFindString(),GetSearchCase(),GetFindWholeWord(), GetReplaceString());
				}
			}
	
		}
	}
}

void UChooserFindProperties::RemoveInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UChooserTable* ChooserTable = Cast<UChooserTable>(Asset))
		{
			Asset->MarkPackageDirty();

			for(FInstancedStruct& Column : ChooserTable->ColumnsStructs)
			{
				if (FChooserColumnBase* ColumnData = Column.GetMutablePtr<FChooserColumnBase>())
				{
					TStringBuilder<256> StringBuilder;
					ColumnData->GetInputValue()->AddSearchNames(StringBuilder);
					
					bool bMatched = false;
					
					UE::String::ParseTokens(StringBuilder, *UChooserTable::PropertyTagDelimiter, [this, &bMatched](FStringView InToken)
					{
						if (NameMatches(InToken))
						{
							bMatched = true;
						}
					}, UE::String::EParseTokensOptions::SkipEmpty);

					if (bMatched)
					{
						// if the name matched, reset the input parameter
						ColumnData->SetInputType(ColumnData->GetInputType());
					}
				}
			}
	
		}
	}
}

void UChooserFindProperties::GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const
{
	for (const FAssetData& AssetData : InAssetDatas)
	{
		const FString TagValue = AssetData.GetTagValueRef<FString>(UChooserTable::PropertyNamesTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *UChooserTable::PropertyTagDelimiter, [&OutUniqueNames](FStringView InToken)
			{
				OutUniqueNames.Add(FString(InToken));
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	}
}

#undef LOCTEXT_NAMESPACE
