// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimAssetFindReplace.h"
#include "ChooserFindProperties.generated.h"

/** Find, replace and remove curves across assets */
UCLASS(MinimalAPI, DisplayName="ChooserProperties")
class UChooserFindProperties : public UAnimAssetFindReplaceProcessor_StringBase
{
	GENERATED_BODY()

	// UAnimAssetFindReplaceProcessor interface
	virtual FString GetFindResultStringFromAssetData(const FAssetData& InAssetData) const override;
	virtual TConstArrayView<UClass*> GetSupportedAssetTypes() const override;
	virtual bool ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const override;
	virtual void ReplaceInAsset(const FAssetData& InAssetData) const override;
	virtual void RemoveInAsset(const FAssetData& InAssetData) const override;

	// UAnimAssetFindReplaceProcessor_StringBase interface
	virtual void GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const override;
};
