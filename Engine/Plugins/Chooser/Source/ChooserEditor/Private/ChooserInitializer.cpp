// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserInitializer.h"

#include "AnimNode_ChooserPlayer.h"
#include "Chooser.h"
#include "ChooserPropertyAccess.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserInitializer)

#define LOCTEXT_NAMESPACE "ChooserInitializer"


void FGenericChooserInitializer::InitializeSignature(UChooserSignature* Chooser) const
{
	Chooser->ContextData = ContextData;
	Chooser->ResultType = ResultType;
	Chooser->OutputObjectType = OutputObjectType;
}

void FChooserPlayerInitializer::InitializeSignature(UChooserSignature* Chooser) const
{
	Chooser->ContextData.SetNum(2);
	Chooser->ContextData[0].InitializeAs(FContextObjectTypeClass::StaticStruct());
	FContextObjectTypeClass& ClassData = Chooser->ContextData[0].GetMutable<FContextObjectTypeClass>();
	if (AnimClass)
	{
		ClassData.Class = AnimClass;		
	}
	else
	{
		ClassData.Class = UAnimInstance::StaticClass();
	}
	ClassData.Direction = EContextObjectDirection::ReadWrite;
	
	Chooser->ContextData[1].InitializeAs(FContextObjectTypeStruct::StaticStruct());
	FContextObjectTypeStruct& StructData = Chooser->ContextData[1].GetMutable<FContextObjectTypeStruct>();
	StructData.Struct = FChooserPlayerSettings::StaticStruct();
	StructData.Direction = EContextObjectDirection::Write;
	
	Chooser->OutputObjectType = UAnimationAsset::StaticClass();
}

void FNoPrimaryResultChooserInitializer::InitializeSignature(UChooserSignature* Chooser) const
{
	Chooser->ContextData = ContextData;
	Chooser->ResultType = EObjectChooserResultType::NoPrimaryResult;

	// Dummy result type
	Chooser->OutputObjectType = UClass::StaticClass();
}


#undef LOCTEXT_NAMESPACE
