// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserProvider.h"
#include "ObjectTrace.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GChooserProviderLockState;
}

FName FChooserProvider::ProviderName("ChooserProvider");

#define LOCTEXT_NAMESPACE "ChooserProvider"

FChooserProvider::FChooserProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

bool FChooserProvider::ReadChooserEvaluationTimeline(uint64 InObjectId, TFunctionRef<void(const ChooserEvaluationTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToChooserEvaluationTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(ChooserEvaluationTimelines.Num()))
		{
			Callback(*ChooserEvaluationTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FChooserProvider::ReadChooserValueTimeline(uint64 InObjectId, TFunctionRef<void(const ChooserValueTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToChooserEvaluationTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(ChooserEvaluationTimelines.Num()))
		{
			Callback(*ChooserValueTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

void FChooserProvider::EnumerateChooserEvaluationTimelines(TFunctionRef<void(uint64 OwnerId, const ChooserEvaluationTimeline&)> Callback) const
{
	ReadAccessCheck();

	for (TTuple<uint64, uint32> Entry : ObjectIdToChooserEvaluationTimelines)
	{
		if (ChooserEvaluationTimelines.IsValidIndex(Entry.Value))
		{
			Callback(Entry.Key, *ChooserEvaluationTimelines[Entry.Value]);
		}
	}
}

uint32 FChooserProvider::GetTimelineIndex(uint64 InObjectId)
{
	TSharedPtr<TraceServices::TPointTimeline<FChooserEvaluationData>> Timeline;
	uint32* IndexPtr = ObjectIdToChooserEvaluationTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		return *IndexPtr;
	}
	else
	{
		uint32 Index = ChooserEvaluationTimelines.Num();
		ObjectIdToChooserEvaluationTimelines.Add(InObjectId, ChooserEvaluationTimelines.Num());
		ChooserEvaluationTimelines.Add( MakeShared<TraceServices::TPointTimeline<FChooserEvaluationData>>(Session.GetLinearAllocator()));
		ChooserValueTimelines.Add(MakeShared<TraceServices::TPointTimeline<FChooserValueData>>(Session.GetLinearAllocator()));
		return Index;
	}
}

void  FChooserProvider::AppendChooserEvaluation(uint64 InChooserId, uint64 InObjectId, TArrayView<const int32> InSelectedIndices, double InRecordingTime)
{
	EditAccessCheck();

	TSharedPtr<TraceServices::TPointTimeline<FChooserEvaluationData>> Timeline = ChooserEvaluationTimelines[GetTimelineIndex(InObjectId)];
	Timeline->AppendEvent(InRecordingTime, { InChooserId, TArray<int32>(InSelectedIndices) });
}





#undef LOCTEXT_NAMESPACE
