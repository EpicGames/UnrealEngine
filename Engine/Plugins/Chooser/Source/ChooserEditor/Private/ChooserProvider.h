// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GChooserProviderLockState;
}

struct FChooserEvaluationData
{
	uint64 ChooserId;
	TArray<int32> SelectedIndices;
};

struct FChooserValueData
{
	uint64 ChooserId;
	FName Key;
	// probably should get this using the linear allocator somehow:
	TArray<uint8> Value;
};

class FChooserProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	typedef TraceServices::ITimeline<FChooserEvaluationData> ChooserEvaluationTimeline;
	typedef TraceServices::ITimeline<FChooserValueData> ChooserValueTimeline;

	static FName ProviderName;

	FChooserProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GChooserProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GChooserProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GChooserProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GChooserProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GChooserProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GChooserProviderLockState);
	}

	bool ReadChooserEvaluationTimeline(uint64 InObjectId, TFunctionRef<void(const ChooserEvaluationTimeline&)> Callback) const;
	bool ReadChooserValueTimeline(uint64 InObjectId, TFunctionRef<void(const ChooserValueTimeline&)> Callback) const;
	void EnumerateChooserEvaluationTimelines(TFunctionRef<void(uint64 OwnerId, const ChooserEvaluationTimeline&)> Callback) const;



	void AppendChooserEvaluation(uint64 InObjectId, uint64 InChooser, TArrayView<const int32> InSelectedIndices, double InRecordingTime);

	void AppendChooserValue(uint64 InChooserId, uint64 InObjectId, double InRecordingTime, const FName& Key, TArrayView<const uint8> SerializedValue)
    {
    	EditAccessCheck();
    
    	TSharedPtr<TraceServices::TPointTimeline<FChooserValueData>> Timeline = ChooserValueTimelines[GetTimelineIndex(InObjectId)];
		FChooserValueData EventData;
		EventData.ChooserId = InChooserId;
		EventData.Key = Key;
		EventData.Value = SerializedValue;
    	Timeline->AppendEvent(InRecordingTime, EventData);
    }

private:
	uint32 GetTimelineIndex(uint64 ObjectId);

	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;

	TMap<uint64, uint32> ObjectIdToChooserEvaluationTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FChooserEvaluationData>>> ChooserEvaluationTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FChooserValueData>>> ChooserValueTimelines;
};
