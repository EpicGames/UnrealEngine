// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserSignatureFactory.h"
#include "Chooser.h"
#include "ChooserInitializer.h"
#include "ChooserEditorSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Editor.h"
#include "ObjectChooserClassFilter.h"
#include "StructViewerModule.h"
#include "Modules/ModuleManager.h"
#include "SChooserCreateDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserSignatureFactory)

UChooserSignatureFactory::UChooserSignatureFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UChooserSignature::StaticClass();
}

bool UChooserSignatureFactory::ConfigureProperties()
{
	TSharedRef<SChooserCreateDialog> Dialog = SNew(SChooserCreateDialog);
	return Dialog->ConfigureProperties(this);
}

UObject* UChooserSignatureFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	UChooserSignature* NewChooserSignature =  NewObject<UChooserSignature>(InParent, Class, Name, Flags);
	if (ChooserInitializer.IsValid())
	{
		ChooserInitializer.Get<FChooserInitializer>().InitializeSignature(NewChooserSignature);
		UChooserEditorSettings::Get().DefaultCreateType = ChooserInitializer.GetScriptStruct()->GetStructPathName().ToString();
	}
	return NewChooserSignature;
}
