// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "StructUtils/InstancedStruct.h"

#include "ChooserSignatureFactory.generated.h"

UCLASS(MinimalAPI)
class UChooserSignatureFactory : public UFactory
{
	GENERATED_BODY()

public:
	UChooserSignatureFactory();
	
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override
	{
		return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
	}
	virtual FName GetNewAssetThumbnailOverride() const override { return "ChooserEditor.ChooserTableIconLarge"; }

	UPROPERTY(EditAnywhere, DisplayName = "Chooser Type", NoClear, Category="Chooser Setup", meta = (ExcludeBaseStruct, BaseStruct = "/Script/ChooserEditor.ChooserInitializer"))
	FInstancedStruct ChooserInitializer;
};