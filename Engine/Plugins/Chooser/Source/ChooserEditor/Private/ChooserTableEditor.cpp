// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTableEditor.h"

#include "Chooser.h"
#include "ChooserDetails.h"
#include "ChooserEditorWidgets.h"
#include "ChooserFindProperties.h"
#include "ChooserTableEditorCommands.h"
#include "ClassViewerFilter.h"
#include "DetailCategoryBuilder.h"
#include "Factories.h"
#include "GraphEditorSettings.h"
#include "IPropertyAccessEditor.h"
#include "LandscapeRender.h"
#include "ObjectChooserClassFilter.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectChooser_Asset.h"
#include "ObjectChooser_Class.h"
#include "PersonaModule.h"
#include "StructUtils/PropertyBag.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "RandomizeColumn.h"
#include "SAssetDropTarget.h"
#include "SChooserColumnHandle.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "SNestedChooserTree.h"
#include "SourceCodeNavigation.h"
#include "StructViewerModule.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "SChooserTableRow.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"
#include "ToolMenus.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Commands/GenericCommands.h"
#include "SPositiveActionButton.h"
#include "SChooserTableWidget.h"

#define LOCTEXT_NAMESPACE "ChooserEditor"

namespace UE::ChooserEditor
{
	
constexpr int32 HistorySize = 16;	

const FName FChooserTableEditor::ToolkitFName( TEXT( "ChooserTableEditor" ) );
const FName FChooserTableEditor::PropertiesTabId( TEXT( "ChooserEditor_Properties" ) );
const FName FChooserTableEditor::FindReplaceTabId( TEXT( "ChooserEditor_FindReplace" ) );
const FName FChooserTableEditor::TableTabId( TEXT( "ChooserEditor_Table" ) );
const FName FChooserTableEditor::NestedTablesTreeTabId( TEXT( "ChooserEditor_NestedTables" ) );

void FChooserTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_ChooserTableEditor", "Chooser Table Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( PropertiesTabId, FOnSpawnTab::CreateSP(this, &FChooserTableEditor::SpawnPropertiesTab) )
		.SetDisplayName( LOCTEXT("PropertiesTab", "Details") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon("EditorStyle", "LevelEditor.Tabs.Details"));
		
	InTabManager->RegisterTabSpawner( TableTabId, FOnSpawnTab::CreateSP(this, &FChooserTableEditor::SpawnTableTab) )
		.SetDisplayName( LOCTEXT("TableTab", "Chooser Table") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon("ChooserEditorStyle", "ChooserEditor.ChooserTableIconSmall"));
		
	InTabManager->RegisterTabSpawner( NestedTablesTreeTabId, FOnSpawnTab::CreateSP(this, &FChooserTableEditor::SpawnNestedTablesTreeTab) )
		.SetDisplayName( LOCTEXT("NestedTablesTab", "Nested Choosers") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon("ChooserEditorStyle", "ChooserEditor.ChooserTableIconSmall"));


	InTabManager->RegisterTabSpawner( FindReplaceTabId, FOnSpawnTab::CreateSP(this, &FChooserTableEditor::SpawnFindReplaceTab) )
		.SetDisplayName( LOCTEXT("FindReplaceTab", "Find/Replace") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Find"));
}
	
void FChooserTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( TableTabId );
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
	InTabManager->UnregisterTabSpawner( FindReplaceTabId );
}

const FName FChooserTableEditor::ChooserEditorAppIdentifier( TEXT( "ChooserEditorApp" ) );

FChooserTableEditor::FChooserTableEditor()
{
}

FChooserTableEditor::~FChooserTableEditor()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

FName FChooserTableEditor::EditorName = "ChooserTableEditor";
	
FName FChooserTableEditor::ContextMenuName("ChooserEditorContextMenu");
	
FName FChooserTableEditor::GetEditorName() const
{
	return EditorName;
}



UChooserTable* FChooserTableEditor::GetRootChooser()
{
	return ViewModel->GetRootChooser();
}

void FChooserTableEditor::RegisterMenus()
{
	ViewModel->RegisterMenus(GetToolkitCommands());
	
	struct Local
	{
		static void FillEditMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("ChooserEditing", LOCTEXT("Chooser Table Editing", "Chooser Table"));
			{
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut, NAME_None);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate, NAME_None, LOCTEXT("Duplicate Selection", "Duplicate Selection"));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("Delete Selection", "Delete Selection"));
				MenuBuilder.AddMenuEntry(FChooserTableEditorCommands::Get().Disable, NAME_None, LOCTEXT("Disable Selection", "Disable Selection"));
				MenuBuilder.AddMenuEntry(FChooserTableEditorCommands::Get().RemoveDisabledData, NAME_None);
			}
			MenuBuilder.EndSection();
		}
	};
	
	TSharedPtr<FExtender> MenuExtender = MakeShareable(new FExtender);

	// Extend the Edit menu
	MenuExtender->AddMenuExtension(
		"EditHistory",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::FillEditMenu));

	AddMenuExtender(MenuExtender);
}

void FChooserTableEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);
	UChooserEditorToolMenuContext* Context = NewObject<UChooserEditorToolMenuContext>();
	Context->ViewModel = ViewModel;
	MenuContext.AddObject(Context);
	
	MenuContext.AppendCommandList(ToolkitCommands);
}
	
void FChooserTableEditor::SaveAsset_Execute()
{
	ViewModel->AutoPopulateAll();
	FAssetEditorToolkit::SaveAsset_Execute();
}

void FChooserTableEditor::InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	UChooserTable* Chooser = Cast<UChooserTable>(ObjectsToEdit[0]);
	
	ViewModel = MakeShared<FChooserTableViewModel>(Chooser);
	ViewModel->SetOpenObjectDelegate(FOpenObject::CreateLambda([this](UObject* Object)
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(Object))
		{
			SetChooserTableToEdit(Chooser);
		}
	}));

	History.Reserve(HistorySize);
	
	BreadcrumbTrail = SNew(SBreadcrumbTrail<UChooserTable*>)
		.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
		.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
		.ButtonContentPadding( FMargin(4.f, 2.f) )
		.DelimiterImage( FAppStyle::GetBrush("BreadcrumbTrail.Delimiter") )
		.OnCrumbPushed_Lambda([this](UChooserTable* Table)
		{
			ViewModel->SetChooser(BreadcrumbTrail->PeekCrumb());
		})
		.OnCrumbClicked_Lambda([this](UChooserTable* Table)
		{
			AddHistory();
			ViewModel->SetChooser(BreadcrumbTrail->PeekCrumb());
		})
		.GetCrumbMenuContent_Lambda([this](UChooserTable* Item)
		{
			return MakeChoosersMenu(Item);
		})
	;

	BreadcrumbTrail->PushCrumb(FText::FromString(Chooser->GetName()), Chooser);
	AddHistory();

	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FChooserTableEditor::OnObjectsReplaced);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = ViewModel.Get();
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	ViewModel->SetShowDetailsDelegate(FShowDetails::CreateLambda([DetailsViewWeakPtr = DetailsView.ToWeakPtr()](const TArray<UObject*>& Objects)
		{
			if (TSharedPtr<IDetailsView> Details = DetailsViewWeakPtr.Pin())
			{
				Details->SetObjects(Objects, true);
			}
		}));
	
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_ChooserTableEditor_Layout_v1.6" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab( TableTabId, ETabState::OpenedTab )
			)
			->Split
			(
			FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.3f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab( PropertiesTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab( NestedTablesTreeTabId, ETabState::OpenedTab )
				)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FChooserTableEditor::ChooserEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

	ViewModel->RegisterMenus(GetToolkitCommands());
	
	RegenerateMenusAndToolbars();

	ViewModel->SelectRootProperties();
	SetChooserTableToEdit(Chooser);
		
	FAnimAssetFindReplaceConfig FindReplaceConfig;
	FindReplaceConfig.InitialProcessorClass = UChooserFindProperties::StaticClass();
}

void FChooserTableEditor::FocusWindow(UObject* ObjectToFocusOn)
{
	if (UChooserTable* Chooser = Cast<UChooserTable>(ObjectToFocusOn))
	{
		SetChooserTableToEdit(Chooser);
	}
	FAssetEditorToolkit::FocusWindow(ObjectToFocusOn);
}

FName FChooserTableEditor::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FChooserTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Chooser Table Editor");
}

void FChooserTableEditor::RefreshNestedObjectTree()
{
	if (NestedChooserTree.IsValid())
	{
		NestedChooserTree->RefreshAll();
	}
}

FText FChooserTableEditor::GetToolkitName() const
{
	check( ViewModel->GetRootChooser() );
	return FText::FromString(ViewModel->GetRootChooser()->GetName());
}

FText FChooserTableEditor::GetToolkitToolTipText() const
{
	check( ViewModel->GetRootChooser() );
	return FAssetEditorToolkit::GetToolTipTextForObject(ViewModel->GetRootChooser());
}

FLinearColor FChooserTableEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.0f, 0.0f, 0.5f );
}

void FChooserTableEditor::SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate)
{
	DetailsView->SetIsPropertyVisibleDelegate(InVisibilityDelegate);
	DetailsView->ForceRefresh();
}

void FChooserTableEditor::SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate)
{
	DetailsView->SetIsPropertyEditingEnabledDelegate(InPropertyEditingDelegate);
	DetailsView->ForceRefresh();
}


TSharedRef<SDockTab> FChooserTableEditor::SpawnPropertiesTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == PropertiesTabId );

	return SNew(SDockTab)
		.Label( LOCTEXT("GenericDetailsTitle", "Details") )
		.TabColorScale( GetTabColorScale() )
		.OnCanCloseTab_Lambda([]() { return false; })
		[
			DetailsView.ToSharedRef()
		];
}
	
TSharedRef<SDockTab> FChooserTableEditor::SpawnFindReplaceTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == FindReplaceTabId );

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	FAnimAssetFindReplaceConfig Config;
	Config.InitialProcessorClass = UChooserFindProperties::StaticClass();
	return SNew(SDockTab)
		.Label( LOCTEXT("FindReplaceTitle", "Find/Replace") )
		.TabColorScale( GetTabColorScale() )
	[
		PersonaModule.CreateFindReplaceWidget(Config)
	];
}
	
TSharedRef<SDockTab> FChooserTableEditor::SpawnNestedTablesTreeTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == NestedTablesTreeTabId );

	return SNew(SDockTab)
		.Label( LOCTEXT("NestedChooserTreeTitle", "Nested Choosers") )
		[
			SAssignNew(NestedChooserTree, SNestedChooserTree).ChooserEditor(this)
		];
}

TSharedRef<SDockTab> FChooserTableEditor::SpawnTableTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == TableTabId );

	UChooserTable* Chooser = ViewModel->GetChooser();

	TSharedRef<SWidget> ChooserTableView = SNew(SChooserTableWidget)
											.Commands(GetToolkitCommands())
											.ViewModel(ViewModel);

	ViewModel->RefreshAll();

	TSharedRef<SComboButton> EditChooserTableButton = SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton");
	
	EditChooserTableButton->SetOnGetMenuContent(
    		FOnGetContent::CreateLambda(
    			[this]()
                		{
							return MakeChoosersMenu(ViewModel->GetRootChooser()->GetPackage());
                		})
    		);

	return SNew(SDockTab)
		.Label( LOCTEXT("ChooserTableTitle", "Chooser Table") )
		.TabColorScale( GetTabColorScale() )
		.OnCanCloseTab_Lambda([]() { return false; })
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(3)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
						.IsEnabled(this, &FChooserTableEditor::CanNavigateBack)
						.OnClicked_Lambda([this]()
						{
							NavigateBack();
							return FReply::Handled();
						})
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.ArrowLeft"))
						]
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
						.IsEnabled(this, &FChooserTableEditor::CanNavigateForward)
						.OnClicked_Lambda([this]()
						{
							NavigateForward();
							return FReply::Handled();
						})
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight") )
						]
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					EditChooserTableButton
				]
				+ SHorizontalBox::Slot().FillWidth(1)
				[
					BreadcrumbTrail.ToSharedRef()
				]

				
			]
			+ SVerticalBox::Slot().FillHeight(1)
			[
					ChooserTableView
			]
		];
}

void FChooserTableEditor::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bChangedAny = false;

	UChooserTable* RootChooser = ViewModel->GetRootChooser();

	UObject* ReplacedObject = ReplacementMap.FindRef(RootChooser);

	if (ReplacedObject && ReplacedObject != RootChooser)
	{
		RootChooser = Cast<UChooserTable>(ReplacedObject);
		SetChooserTableToEdit(RootChooser);
		ViewModel->SelectRootProperties();
	}
}

FString FChooserTableEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Chooser Table Asset ").ToString();
}

TSharedRef<FChooserTableEditor> FChooserTableEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FChooserTableEditor > NewEditor( new FChooserTableEditor() );

	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add( ObjectToEdit );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );

	return NewEditor;
}

TSharedRef<FChooserTableEditor> FChooserTableEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FChooserTableEditor > NewEditor( new FChooserTableEditor() );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );
	return NewEditor;
}


void FChooserTableEditor::AddHistory()
{
	// remove anything ahead of this in the history, if we had gone back
	while (HistoryIndex !=0)
	{
		History.PopFront();
		HistoryIndex--;
	}
	
	if (History.Num() >= HistorySize)
	{
		History.Pop();
	}
	History.AddFront(ViewModel->GetChooser());
}

bool FChooserTableEditor::CanNavigateBack() const
{
	return HistoryIndex < History.Num() - 1;
}

void FChooserTableEditor::NavigateBack()
{
	if (HistoryIndex < History.Num() - 1)
	{
		HistoryIndex++;
		SetChooserTableToEdit(History[HistoryIndex], false);
	}
}

bool FChooserTableEditor::CanNavigateForward() const
{
	return HistoryIndex > 0;
}

void FChooserTableEditor::NavigateForward()
{
	if (HistoryIndex > 0)
	{
		HistoryIndex--;
		SetChooserTableToEdit(History[HistoryIndex], false);
	}
}

void FChooserTableEditor::SetChooserTableToEdit(UChooserTable* Chooser, bool bApplyToHistory)
{
	if (Chooser == ViewModel->GetChooser())
	{
		return;
	}
	
	BreadcrumbTrail->ClearCrumbs();

	TArray<UChooserTable*> OuterList;
	OuterList.Push(Chooser);
	
	while(OuterList.Last() != ViewModel->GetRootChooser())
	{
		OuterList.Push(Cast<UChooserTable>(OuterList.Last()->GetOuter()));
	}

	while(!OuterList.IsEmpty())
	{
		UChooserTable* Popped = OuterList.Pop();
		BreadcrumbTrail->PushCrumb(FText::FromString(Popped->GetName()), Popped);
	}
	
	if (bApplyToHistory)
	{
		AddHistory();
	}
	
	ViewModel->SetChooser(Chooser);
}

void FChooserTableEditor::PushChooserTableToEdit(UChooserTable* Chooser)
{
	BreadcrumbTrail->PushCrumb(FText::FromString(Chooser->GetName()), Chooser);
	AddHistory();
	ViewModel->SetChooser(Chooser);
}
	
void FChooserTableEditor::PopChooserTableToEdit()
{
	if (BreadcrumbTrail->HasCrumbs())
	{
		BreadcrumbTrail->PopCrumb();
		ViewModel->SetChooser(BreadcrumbTrail->PeekCrumb());
	}
}

	
void FChooserTableEditor::MakeChoosersMenuRecursive(UObject* Outer, FMenuBuilder& MenuBuilder, const FString& Indent = "") 
{
	TArray<UObject*> ChildObjects;
	GetObjectsWithOuter(Outer, ChildObjects, EGetObjectsFlags::None);

	FString SubIndent = Indent + "    ";
	for (UObject* Object : ChildObjects)
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(Object))
		{
			if (Chooser == ViewModel->GetRootChooser() || Chooser->GetRootChooser()->NestedObjects.Contains(Chooser))
			{
				MenuBuilder.AddMenuEntry( FText::FromString(Indent + Chooser->GetName()), LOCTEXT("Edit Chooser ToolTip", "Browse to this Nested Chooser Table"), FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this, Chooser]()
					{
						SetChooserTableToEdit(Chooser);
					})));

				MakeChoosersMenuRecursive(Chooser, MenuBuilder, SubIndent);
			}
		}
	}
}
	
TSharedRef<SWidget> FChooserTableEditor::MakeChoosersMenu(UObject* RootObject)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MakeChoosersMenuRecursive(RootObject, MenuBuilder);

	return MenuBuilder.MakeWidget();
}

	
	
void FChooserTableEditor::RegisterWidgets()
{
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FAssetChooser::StaticStruct(), CreateAssetWidget);
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FSoftAssetChooser::StaticStruct(), CreateSoftAssetWidget);
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FClassChooser::StaticStruct(), CreateClassWidget);
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FEvaluateChooser::StaticStruct(), CreateEvaluateChooserWidget);
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FNestedChooser::StaticStruct(), CreateNestedChooserWidget);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	PropertyModule.RegisterCustomClassLayout("ChooserTable", FOnGetDetailCustomizationInstance::CreateStatic(&FChooserDetails::MakeInstance));	
	PropertyModule.RegisterCustomClassLayout("ChooserRowDetails", FOnGetDetailCustomizationInstance::CreateStatic(&FChooserRowDetails::MakeInstance));	
	PropertyModule.RegisterCustomClassLayout("ChooserColumnDetails", FOnGetDetailCustomizationInstance::CreateStatic(&FChooserColumnDetails::MakeInstance));	
}

}

#undef LOCTEXT_NAMESPACE

