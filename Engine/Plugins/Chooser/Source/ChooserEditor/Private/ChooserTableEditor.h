// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chooser.h"
#include "EditorUndoClient.h"
#include "PropertyEditorDelegates.h"
#include "SNestedChooserTree.h"
#include "Containers/RingBuffer.h"
#include "Misc/NotifyHook.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Views/SListView.h"
#include "ChooserTableViewModel.h"

class SComboButton;
class SPositiveActionButton;
class SEditableText;
class IDetailsView;
class UChooserRowDetails;

namespace UE::ChooserEditor
{
	class SChooserTableWidget;
	class SNestedChooserTree;
	struct FChooserTableRow;

	class FChooserTableEditor : public FAssetEditorToolkit
	{
	public:
		
		/** Delegate that, given an array of assets, returns an array of objects to use in the details view of an FSimpleAssetEditor */
		DECLARE_DELEGATE_RetVal_OneParam(TArray<UObject*>, FGetDetailsViewObjects, const TArray<UObject*>&);

		virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

		/**
		* Edits the specified asset object
		*
		* @param	Mode					Asset editing mode for this editor (standalone or world-centric)
		* @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
		* @param	ObjectsToEdit			The object to edit
		* @param	GetDetailsViewObjects	If bound, a delegate to get the array of objects to use in the details view; uses ObjectsToEdit if not bound
		*/
		void InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects );

		virtual void FocusWindow(UObject* ObjectToFocusOn) override;

		FChooserTableEditor();
		
		/** Destructor */
		virtual ~FChooserTableEditor();

		virtual FName GetEditorName() const override;

		/** IToolkit interface */
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitName() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual bool IsPrimaryEditor() const override { return true; }
		virtual bool IsSimpleAssetEditor() const override { return false; }
		virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
		virtual void SaveAsset_Execute() override;

		virtual FName GetToolMenuToolbarName(FName& OutParentName) const override
		{
			return IChooserTableViewModel::ChooserToolbarName;
		}


		/** Used to show or hide certain properties */
		void SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate);
		/** Can be used to disable the details view making it read-only */
		void SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate);
		
		UChooserTable* GetRootChooser();
		void PushChooserTableToEdit(UChooserTable* Chooser);
		void SetChooserTableToEdit(UChooserTable* Chooser, bool bApplyToHistory = true);
	private:
		void AddHistory();
		bool CanNavigateBack() const;
		void NavigateBack();
		bool CanNavigateForward() const;
		void NavigateForward();
		void PopChooserTableToEdit();
		
		void RegisterMenus();
			
		/** Create the properties tab and its content */
		TSharedRef<SDockTab> SpawnPropertiesTab( const FSpawnTabArgs& Args );
		/** Create the table tab and its content */
		TSharedRef<SDockTab> SpawnTableTab( const FSpawnTabArgs& Args );
		/** Create the find/replace tab and its content */
		TSharedRef<SDockTab> SpawnFindReplaceTab( const FSpawnTabArgs& Args );
		/** Create the nested tables list tab and its content */
		TSharedRef<SDockTab> SpawnNestedTablesTreeTab( const FSpawnTabArgs& Args );
	
		/** Called when objects need to be swapped out for new versions, like after a blueprint recompile. */
		void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

		/** Details view */
		TSharedPtr< class IDetailsView > DetailsView;

	
		/** App Identifier. */
		static const FName ChooserEditorAppIdentifier;

		/**	The tab ids for all the tabs used */
		static const FName PropertiesTabId;
		static const FName FindReplaceTabId;
		static const FName TableTabId;
		static const FName NestedTablesTreeTabId;
		
		void RefreshNestedObjectTree();
	
		TSharedPtr<IChooserTableViewModel> ViewModel;
		
		TSharedPtr<SChooserTableWidget> ChooserTableWidget;
		
		TRingBuffer<UChooserTable*> History;
		int32 HistoryIndex = 0;
		TSharedPtr<SBreadcrumbTrail<UChooserTable*>> BreadcrumbTrail;
		
		
		TSharedRef<SWidget> MakeChoosersMenu(UObject* RootObject);
		void MakeChoosersMenuRecursive(UObject* Outer, FMenuBuilder& MenuBuilder, const FString& Indent);
		
	public:

		TSharedPtr<SNestedChooserTree> NestedChooserTree;

		/** The name given to all instances of this type of editor */
		static const FName ToolkitFName;

		static TSharedRef<FChooserTableEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );

		static TSharedRef<FChooserTableEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );

		static void RegisterWidgets();
		
		static FName EditorName;
		static FName ContextMenuName;
	};

}
