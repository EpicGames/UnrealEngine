// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTableViewModel.h"

#include "Chooser.h"
#include "ChooserDetails.h"
#include "ChooserEditorWidgets.h"
#include "ChooserFindProperties.h"
#include "ChooserTableEditorCommands.h"
#include "ClassViewerFilter.h"
#include "DetailCategoryBuilder.h"
#include "Factories.h"
#include "GraphEditorSettings.h"
#include "IPropertyAccessEditor.h"
#include "LandscapeRender.h"
#include "ObjectChooserClassFilter.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectChooser_Asset.h"
#include "ObjectChooser_Class.h"
#include "PersonaModule.h"
#include "StructUtils/PropertyBag.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "SAssetDropTarget.h"
#include "SChooserColumnHandle.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "SNestedChooserTree.h"
#include "SourceCodeNavigation.h"
#include "StructViewerModule.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "SChooserTableRow.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"
#include "ToolMenus.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Commands/GenericCommands.h"
#include "SPositiveActionButton.h"
#include "ChooserTableEditor.h"
#include "SChooserTableWidget.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChooserTableViewModel)

#define LOCTEXT_NAMESPACE "ChooserEditor"

namespace UE::ChooserEditor
{

const FName IChooserTableViewModel::ChooserToolbarName = "ChooserTableToolbar";

FChooserTableViewModel::~FChooserTableViewModel()
{
	FTSTicker::RemoveTicker(TickDelegateHandle);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
}

void FChooserTableViewModel::RegisterMenus(TSharedPtr<FUICommandList> CommandList)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	const FChooserTableEditorCommands& Commands = FChooserTableEditorCommands::Get();

	// Table Context Menu
	UToolMenu* ToolMenu;
	if (ToolMenus->IsMenuRegistered(FChooserTableEditor::ContextMenuName))
	{
		ToolMenu = ToolMenus->ExtendMenu(FChooserTableEditor::ContextMenuName);
	}
	else
	{
		ToolMenu = UToolMenus::Get()->RegisterMenu(FChooserTableEditor::ContextMenuName, NAME_None, EMultiBoxType::Menu);
	}

	if (ToolMenu)
	{
		FToolMenuSection& Section = ToolMenu->AddSection("ChooserEditorContext", TAttribute<FText>());
	
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Copy));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Cut));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Paste));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Duplicate));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Delete));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.Disable));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.MoveUp));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.MoveDown));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.MoveLeft));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.MoveRight));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(Commands.AutoPopulateSelection));
	}
}

void FChooserTableViewModel::OnObjectsTransacted(UObject* Object, const FTransactionObjectEvent& Event)
{
	if (UChooserTable* ChooserTable = Cast<UChooserTable>(Object))
	{
		// if this is the chooser we're editing
		if (GetChooser() == ChooserTable)
		{
			if (GetCurrentSelectionType() == FChooserTableViewModel::ESelectionType::Rows)
			{
				// refresh details if we have rows selected
				RefreshRowSelectionDetails();
			}
		}
	}
	
	if (UChooserRowDetails* RowDetails = Cast<UChooserRowDetails>(Object))
	{
		// if this is for the chooser we're editing
		if (GetChooser() == RowDetails->Chooser)
		{
			QueueRowTransaction(RowDetails);
			
		}
	}
}
	
bool FChooserTableViewModel::HandleTicker(float DeltaTime)
{
	if (!QueuedRowTransactions.IsEmpty())
	{
		for (TWeakObjectPtr<UChooserRowDetails> RowDetails : QueuedRowTransactions)
		{
			if (RowDetails.IsValid())
			{
				if (RowDetails->Chooser->ResultsStructs.IsValidIndex(RowDetails->Row))
				{
					// copy all the values over
					TValueOrError<FStructView, EPropertyBagResult> Result = RowDetails->Properties.GetValueStruct("Result", FInstancedStruct::StaticStruct());
					if (Result.IsValid())
					{
						RowDetails->Chooser->ResultsStructs[RowDetails->Row] = Result.GetValue().Get<FInstancedStruct>();
					}

					int32 ColumnIndex = 0;
					for (FInstancedStruct& ColumnData : RowDetails->Chooser->ColumnsStructs)
					{
						FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
						Column.SetFromDetails(RowDetails->Properties, ColumnIndex, RowDetails->Row);
						ColumnIndex++;
					}
			
					TValueOrError<bool, EPropertyBagResult> DisabledResult = RowDetails->Properties.GetValueBool("Disabled");
					if (DisabledResult.IsValid())
					{
						RowDetails->Chooser->DisabledRows[RowDetails->Row] = DisabledResult.GetValue();
					}
				}
				else if (RowDetails->Row == ColumnWidget_SpecialIndex_Fallback)
				{
					TValueOrError<FStructView, EPropertyBagResult> Result = RowDetails->Properties.GetValueStruct("Result", FInstancedStruct::StaticStruct());
					if (Result.IsValid())
					{
						RowDetails->Chooser->FallbackResult = Result.GetValue().Get<FInstancedStruct>();
					}
				
					int32 ColumnIndex = 0;
					for (FInstancedStruct& ColumnData : RowDetails->Chooser->ColumnsStructs)
					{
						FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
						Column.SetFromDetails(RowDetails->Properties, ColumnIndex, RowDetails->Row);
						ColumnIndex++;
					}
				}
			}
		}
		
		GetChooser()->MarkPackageDirty();
		RefreshAll();
		
		QueuedRowTransactions.Empty();
	}
	return true;
}

FChooserTableViewModel::FChooserTableViewModel(UChooserTable* Chooser)
{
	check(Chooser);

	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FChooserTableViewModel::HandleTicker), 1.f);
	
	RootChooser = Chooser->GetRootChooser();
	CurrentChooser = Chooser;

	FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FChooserTableViewModel::OnObjectsTransacted);
}

void FChooserTableViewModel::RefreshAll()
{
	// Cache Selection state
	ESelectionType CachedSelectionType = CurrentSelectionType;
	int32 SelectedColumnIndex = -1;
	UChooserTable* SelectedChooser = nullptr;
	TArray<int32> CachedSelectedRows;

	if (CachedSelectionType == ESelectionType::Column)
	{
		SelectedColumnIndex = SelectedColumn->Column;
		SelectedChooser = SelectedColumn->Chooser;
	}
	else if (CachedSelectionType == ESelectionType::Rows)
	{
		if (!SelectedRows.IsEmpty())
		{
			SelectedChooser = SelectedRows[0]->Chooser;
		}
		for(const TObjectPtr<UChooserRowDetails>& SelectedRow : SelectedRows)
		{
			CachedSelectedRows.Add(SelectedRow->Row);
		}
	}
	
	UpdateTableColumns();
	UpdateTableRows();

	// reapply cached selection state
	if (CachedSelectionType == ESelectionType::Root)
	{
		SelectRootProperties();
	}
	else if (CachedSelectionType == ESelectionType::Column)
	{
		SelectColumn(SelectedChooser, SelectedColumnIndex);
	}
	else if (CachedSelectionType == ESelectionType::Rows)
	{
		SelectRows(CachedSelectedRows);
	}

	RefreshAssetRegistry();
}

bool FChooserTableViewModel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	TArray<UObject*> ContainedObjects;
	GetObjectsWithOuter(GetRootChooser()->GetPackage(), ContainedObjects, EGetObjectsFlags::IncludeNestedObjects);
	
	for(const TPair<UObject*, FTransactionObjectEvent>&  Entry : TransactionObjectContexts)
	{
		if (ContainedObjects.Contains(Entry.Key))
		{
			return true;
		}
	}
	return false;
}

void FChooserTableViewModel::PostUndo(bool bSuccess)
{
	SelectRootProperties();
	RefreshAll();
	RootChooser->NestedObjectsChanged.Broadcast();
}

void FChooserTableViewModel::PostRedo(bool bSuccess)
{
	SelectRootProperties();
	RefreshAll();
	RootChooser->NestedObjectsChanged.Broadcast();
} 


void FChooserTableViewModel::NotifyPreChange(FProperty* PropertyAboutToChange)
{
}

void FChooserTableViewModel::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// Called on details panel edits
	
	if (CurrentSelectionType==ESelectionType::Root)
	{
		// Editing the root in the details panel can change ContextData that means all wigets need to be refreshed
		UpdateTableColumns();
		UpdateTableRows();
		SelectRootProperties();
	}
	if (CurrentSelectionType==ESelectionType::Column)
	{
		check(SelectedColumn);
		int32 SelectedColumnIndex = SelectedColumn->Column;
		UChooserTable* SelectedColumnChooser = SelectedColumn->Chooser;
		// Editing column properties can change the column type, which requires refreshing everything
		UpdateTableColumns();
		UpdateTableRows();
		SelectColumn(SelectedColumnChooser, SelectedColumnIndex);
	}
	// editing row data should not require any refreshing
}

void FChooserTableViewModel::RemoveDisabledData()
{
	UChooserTable* Chooser = GetChooser();
	const FScopedTransaction Transaction(LOCTEXT("Remove Disabled Data", "Remove Disabled Data"));

	Chooser->Modify(true);
	Chooser->RemoveDisabledData();
	RefreshAll();
}

int32 FChooserTableViewModel::MoveColumn(int32 SourceIndex, int32 TargetIndex)
{
	UChooserTable* Chooser = GetChooser();
	
	TargetIndex = FMath::Clamp(TargetIndex, 0, Chooser->ColumnsStructs.Num());

	if (SourceIndex < 0 || SourceIndex == TargetIndex)
	{
		return TargetIndex;
	}

	const FScopedTransaction Transaction(LOCTEXT("Move Row", "Move Row"));

	Chooser->Modify(true);

	FInstancedStruct ColumnData = MoveTemp(Chooser->ColumnsStructs[SourceIndex]);
	Chooser->ColumnsStructs.RemoveAt(SourceIndex);
		
	if (SourceIndex < TargetIndex)
	{
		TargetIndex--;
	}
	
	if (TargetIndex == Chooser->ColumnsStructs.Num() && !Chooser->ColumnsStructs.IsEmpty())
	{
		if (Chooser->ColumnsStructs.Last().Get<FChooserColumnBase>().IsRandomizeColumn())
		{
			// never drop after a Randomize Column;
			TargetIndex--;
		}
	}

	Chooser->ColumnsStructs.Insert(MoveTemp(ColumnData), TargetIndex);

	RefreshAll();

	return TargetIndex;
}

void FChooserTableViewModel::UpdateTableColumns()
{
	RefreshTableColumnsDelegate.ExecuteIfBound();
}

void FChooserTableViewModel::QueueRowTransaction(UChooserRowDetails* RowDetails)
{
	QueuedRowTransactions.Add(RowDetails);
}

void FChooserTableViewModel::AddColumn(const UScriptStruct* ColumnType)
{
	FSlateApplication::Get().DismissAllMenus();
	UChooserTable* Chooser = GetChooser();
	const FScopedTransaction Transaction(LOCTEXT("Add Column Transaction", "Add Column"));
	Chooser->Modify(true);

	FInstancedStruct NewColumn;
	NewColumn.InitializeAs(ColumnType);
	FChooserColumnBase& NewColumnRef = NewColumn.GetMutable<FChooserColumnBase>();
	NewColumnRef.Initialize(Chooser);

	int32 InsertIndex = 0;
	if (NewColumnRef.IsRandomizeColumn())
	{
		// add randomization column at the end (do nothing if there already is one)
		InsertIndex = Chooser->ColumnsStructs.Num();
		if (InsertIndex == 0 || !Chooser->ColumnsStructs[InsertIndex - 1].Get<FChooserColumnBase>().IsRandomizeColumn())
		{
			Chooser->ColumnsStructs.Add(MoveTemp(NewColumn));
		}
	}
	else if (NewColumnRef.HasOutputs())
	{
		// add output columns at the end (but before any randomization column)
		InsertIndex = Chooser->ColumnsStructs.Num();
		if (InsertIndex > 0 && Chooser->ColumnsStructs[InsertIndex - 1].Get<FChooserColumnBase>().IsRandomizeColumn())
		{
			InsertIndex--;
		}
		Chooser->ColumnsStructs.Insert(MoveTemp(NewColumn), InsertIndex);
	}
	else
	{
		// add other columns after the last non-output, non-randomization column
		while(InsertIndex < Chooser->ColumnsStructs.Num())
		{
			const FChooserColumnBase& Column = Chooser->ColumnsStructs[InsertIndex].Get<FChooserColumnBase>();
			if (Column.HasOutputs() || Column.IsRandomizeColumn())
			{
				break;
			}
			InsertIndex++;
		}
		Chooser->ColumnsStructs.Insert(MoveTemp(NewColumn), InsertIndex);
	}

	UpdateTableColumns();
	UpdateTableRows();

	SelectColumn(Chooser, InsertIndex);
}

void FChooserTableViewModel::RefreshRowSelectionDetails()
{
	SelectedRows.SetNum(0, EAllowShrinking::No);
	UChooserTable* Chooser = GetChooser();

	FPropertyBagPropertyDesc ResultPropertyDesc ("Result", EPropertyBagPropertyType::Struct, FInstancedStruct::StaticStruct());
	ResultPropertyDesc.MetaData.Add({"ExcludeBaseStruct",""});
	ResultPropertyDesc.MetaData.Add({"BaseStruct","/Script/Chooser.ObjectChooserBase"});
	
	if (SelectedRowsCache.Num() < SelectedRowIndices.Num())
	{
		SelectedRowsCache.SetNum(SelectedRowIndices.Num());
	}
	
	int32 SelectedRowsCacheIndex = 0;
	for(int32 SelectedRowIndex : SelectedRowIndices)
	{
		if (Chooser->ResultsStructs.IsValidIndex(SelectedRowIndex) || SelectedRowIndex == ColumnWidget_SpecialIndex_Fallback)
		{
			if (SelectedRowsCache[SelectedRowsCacheIndex] == nullptr)
			{
				SelectedRowsCache[SelectedRowsCacheIndex] = TStrongObjectPtr(NewObject<UChooserRowDetails>(RootChooser, NAME_None, RF_Transactional | RF_Transient));
			}
			TObjectPtr<UChooserRowDetails> Selection = SelectedRowsCache[SelectedRowsCacheIndex].Get();
			SelectedRowsCacheIndex ++;
			
			Selection->Chooser = Chooser;
			Selection->Row = SelectedRowIndex;
			Selection->Properties.Reset();
		
			if (Chooser->ResultType != EObjectChooserResultType::NoPrimaryResult)
			{
				FInstancedStruct& Result = SelectedRowIndex == ColumnWidget_SpecialIndex_Fallback? Chooser->FallbackResult : Chooser->ResultsStructs[SelectedRowIndex];
				Selection->Properties.AddProperties({ResultPropertyDesc});
				Selection->Properties.SetValueStruct("Result", FConstStructView(FInstancedStruct::StaticStruct(), reinterpret_cast<uint8*>(&Result)));
			}

			int32 ColumnIndex = 0;
			for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();

				if (Column.HasOutputs() || SelectedRowIndex != ColumnWidget_SpecialIndex_Fallback)
				{
					// add each column's details, but on the fallback row, only add details for Output columns
					Column.AddToDetails(Selection->Properties, ColumnIndex, SelectedRowIndex);
				}
				ColumnIndex++;
			}

			if (Chooser->DisabledRows.IsValidIndex(SelectedRowIndex))
			{
				Selection->Properties.AddProperty("Disabled", EPropertyBagPropertyType::Bool);
				Selection->Properties.SetValueBool("Disabled", Chooser->DisabledRows[SelectedRowIndex]);
			}

			SelectedRows.Add(Selection);
		}
	}
	
	TArray<UObject*> DetailsObjects;
	for(auto& Item : SelectedRows)
	{
		DetailsObjects.Add(Item.Get());
	}

	ShowDetailsDelegate.ExecuteIfBound( DetailsObjects );
}

void FChooserTableViewModel::UpdateTableRows()
{
	RefreshTableRowsDelegate.ExecuteIfBound();
}

void FChooserTableViewModel::DeleteColumn(int32 Index)
{
	const FScopedTransaction Transaction(LOCTEXT("Delete Column Transaction", "Delete Column"));
	UChooserTable* Chooser = GetChooser();

	if (Index < Chooser->ColumnsStructs.Num())
	{
		Chooser->Modify(true);
		Chooser->ColumnsStructs.RemoveAt(Index);
		UpdateTableColumns();
	}
}
	
void FChooserTableViewModel::DeleteRows(TConstArrayView<int32> RowsToDelete)
{
	const FScopedTransaction Transaction(LOCTEXT("Delete Row Transaction", "Delete Row"));
	DeleteRowsInternal(RowsToDelete, 0);
}

int32 FChooserTableViewModel::DeleteRowsInternal(TConstArrayView<int32> InRowsToDelete, int32 RowIndexToRemember)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		Chooser->Modify();
		
		// sort indices in reverse
		TArray<int32> RowsToDelete;
		RowsToDelete = InRowsToDelete;
		RowsToDelete.Sort(TGreater<>());

		// calling DeleteRows on columns before removing them from FallbackResult / DisabledRows / ResultsStructs,
		// so they are aware of what's gonna be deleted, and access those properties to remove references, etc.
		for(FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
			Column.DeleteRows(RowsToDelete);
		}

		// removing RowsToDelete from FallbackResult / DisabledRows / ResultsStructs
		for(int32 RowIndex : RowsToDelete)
		{
			if (RowIndex == ColumnWidget_SpecialIndex_Fallback)
			{
				Chooser->FallbackResult.Reset();
			}
			else
			{
				if (RowIndex <= RowIndexToRemember)
				{
					RowIndexToRemember--;
				}
				Chooser->ResultsStructs.RemoveAt(RowIndex);
				Chooser->DisabledRows.RemoveAt(RowIndex);
			}
		}

		
		UpdateTableRows();
	}

	return RowIndexToRemember;
}
	
void FChooserTableViewModel::MoveRows(TConstArrayView<int32> RowIndices, int32 TargetIndex)
{
	const FScopedTransaction Transaction(LOCTEXT("Move Row(s)", "Move Row(s)"));
	UChooserTable* RowCopy = CopyRowsInternal(RowIndices);
	TargetIndex = DeleteRowsInternal(RowIndices, TargetIndex);
	PasteInternal(RowCopy, TargetIndex);
}

void FChooserTableViewModel::AutoPopulateColumn(int32 ColumnIndex)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
		{
			const FScopedTransaction Transaction(LOCTEXT("Autopopulate Column", "Autopopulate Column"));
			Chooser->Modify();
			AutoPopulateColumn(Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>());
		}
	}
}
	
void FChooserTableViewModel::AutoPopulateColumn(FChooserColumnBase& Column)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		const int32 RowCount = Chooser->ResultsStructs.Num();
		if (Column.AutoPopulates())
		{
			for (int32 i = 0; i < RowCount; ++i)
			{
				UObject* ReferencedObject = Chooser->ResultsStructs[i].IsValid() ? Chooser->ResultsStructs[i].Get<FObjectChooserBase>().GetReferencedObject() : nullptr;
				Column.AutoPopulate(i, ReferencedObject);
			}

			UObject* ReferencedObject = Chooser->FallbackResult.IsValid() ? Chooser->FallbackResult.Get<FObjectChooserBase>().GetReferencedObject() : nullptr;
			Column.AutoPopulate(ColumnWidget_SpecialIndex_Fallback, ReferencedObject);
		}
	}
}

void FChooserTableViewModel::AutoPopulateRows(TConstArrayView<int32> RowIndices)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		const FScopedTransaction Transaction(LOCTEXT("Auto Populate Selection", "Auto Populate Selection"));
		Chooser->Modify();
		
		for(int32 RowIndex : RowIndices)
		{
			AutoPopulateRow(RowIndex);
		}
	}
}

void FChooserTableViewModel::AutoPopulateRow(int32 Index)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		if (Chooser->ResultsStructs.IsValidIndex(Index) && Chooser->ResultsStructs[Index].IsValid())
		{
			UObject* ReferencedObject = Chooser->ResultsStructs[Index].Get<FObjectChooserBase>().GetReferencedObject();
			for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
				Column.AutoPopulate(Index, ReferencedObject);
			}
		}
		else if (Index == ColumnWidget_SpecialIndex_Fallback && Chooser->FallbackResult.IsValid())
		{
			UObject* ReferencedObject = Chooser->FallbackResult.Get<FObjectChooserBase>().GetReferencedObject();
			for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
				Column.AutoPopulate(Index, ReferencedObject);
			}
		}
	}
}
	
bool FChooserTableViewModel::CanAutoPopulateColumn(int32 ColumnIndex) const
{
	if (const UChooserTable* Chooser = GetChooser())
	{
		if (CurrentSelectionType == ESelectionType::Column && SelectedColumn)
		{
			// when a column is selected, return true if that column supports auto populate
			if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
			{
				return Chooser->ColumnsStructs[ColumnIndex].Get<FChooserColumnBase>().AutoPopulates();
			}	
		}
	}
	return false;
}
	
bool FChooserTableViewModel::CanAutoPopulateRows() const
{
	if (const UChooserTable* Chooser = GetChooser())
	{
		// when rows are selected, return true if any column supports auto populate
		for (const FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			const FChooserColumnBase& Column = ColumnData.Get<FChooserColumnBase>();
			if (Column.AutoPopulates())
			{
				return true;
			}
		}
	}
	
	return false;
}
	
void FChooserTableViewModel::AutoPopulateAll()
{
	if (UChooserTable* Chooser = GetChooser())
	{
		const FScopedTransaction Transaction(LOCTEXT("Auto Populate Chooser", "Auto Populate All"));
		Chooser->Modify();
		for (FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
			AutoPopulateColumn(Column);
		}
	}
}

bool FChooserTableViewModel::HasSelection() const
{
	return HasRowsSelected() || HasColumnSelected();
}

bool FChooserTableViewModel::HasRowsSelected() const
{
	return CurrentSelectionType == ESelectionType::Rows && !SelectedRows.IsEmpty();
}
	
bool FChooserTableViewModel::HasColumnSelected() const
{
	const UChooserTable* Chooser = GetChooser();
	return CurrentSelectionType == ESelectionType::Column && SelectedColumn && Chooser->ColumnsStructs.IsValidIndex(SelectedColumn->Column);
}
	
bool FChooserTableViewModel::IsColumnDisabled(int32 ColumnIndex) const
{
	if (const UChooserTable* Chooser = GetChooser())
	{
		if ( Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
		{
			const FChooserColumnBase& Column = Chooser->ColumnsStructs[ColumnIndex].Get<FChooserColumnBase>();
			return Column.bDisabled;
		}
	}
	return false;
}

bool FChooserTableViewModel::AreRowsDisabled(TConstArrayView<int32> RowIndices) const
{
	if (const UChooserTable* Chooser = GetChooser())
	{
		bool bSomethingEnabled = false;
		for(int32 Row : RowIndices)
		{
			if (!Chooser->IsRowDisabled(Row))
			{
				bSomethingEnabled = true;
				break;
			}
		}
		return !bSomethingEnabled;
	}
	return false;
}
	
void FChooserTableViewModel::ToggleDisableColumn(int32 ColumnIndex)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		const FScopedTransaction Transaction(LOCTEXT("Toggle Disable Column", "Toggle Disable Column"));
		Chooser->Modify();
		
		if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
		{
			FChooserColumnBase& Column = Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
			Column.bDisabled = !Column.bDisabled;
		}
	}
}

void FChooserTableViewModel::ToggleDisableRows(TConstArrayView<int32> RowIndices)
{
	if (UChooserTable* Chooser = GetChooser())
	{
		const FScopedTransaction Transaction(LOCTEXT("Toggle Disable Rows", "Toggle Disable Row(s)"));
		Chooser->Modify();
		
		bool bDisabled = AreRowsDisabled(RowIndices);
		for (int32 Row : RowIndices)
		{
			if (Chooser->DisabledRows.IsValidIndex(Row))
			{
				Chooser->DisabledRows[Row] = !bDisabled;
			}
		}
		RefreshRowSelectionDetails();
	}
}
	
void FChooserTableViewModel::DuplicateColumn(int32 ColumnIndex)
{
	const FScopedTransaction Transaction(LOCTEXT("Duplicate Column", "Duplicate Column"));
	UChooserTable* Chooser = GetChooser();
	if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
	{
		Chooser->Modify();
		FInstancedStruct Column = Chooser->ColumnsStructs[ColumnIndex];
		Chooser->ColumnsStructs.Insert(Column,ColumnIndex);
		// technically unnecessary, but let the column know that it needs to be initialized since it got duplicated
		Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>().Initialize(Chooser);
	}

	UpdateTableColumns();
}
	
void FChooserTableViewModel::DuplicateRows(TConstArrayView<int32> RowIndices)
{
	const FScopedTransaction Transaction(LOCTEXT("Duplicate Row(s)", "Duplicate Row(s)"));
	UChooserTable* Chooser = GetChooser();
	Chooser->Modify();
	
	int32 MaxSelectedRow = -1;
	for(int32 RowIndex : RowIndices)
	{
		MaxSelectedRow = FMath::Max(RowIndex, MaxSelectedRow);
	}
	
	if (MaxSelectedRow>=0)
	{
		UChooserTable* RowCopy = CopyRowsInternal(RowIndices);
		PasteInternal(RowCopy, MaxSelectedRow + 1);
	}
}

bool FChooserTableViewModel::HasFallbackSelected()
{
	for(UChooserRowDetails* SelectedRow : SelectedRows)
	{
		if (SelectedRow->Row == ColumnWidget_SpecialIndex_Fallback)
		{
			return true;
		}
	}

	return false;
}

bool FChooserTableViewModel::CanMoveRowsUp(TConstArrayView<int32> RowIndices)
{
	UChooserTable* Chooser = GetChooser();

	int32 MinSelectedRow = Chooser->ResultsStructs.Num();
	for(int32 RowIndex : RowIndices)
	{
		if (RowIndex != ColumnWidget_SpecialIndex_Fallback)
		{
			MinSelectedRow = FMath::Min(RowIndex, MinSelectedRow);
		}
	}

	return MinSelectedRow > 0;
}

int32 FChooserTableViewModel::GetNumRows() const
{
	if (const UChooserTable* Chooser = GetChooser())
	{
		return Chooser->ResultsStructs.Num();
	}
	return 0;
}

bool FChooserTableViewModel::CanMoveRowsDown(TConstArrayView<int32> RowIndices)
{
	UChooserTable* Chooser = GetChooser();
	
	int32 MaxSelectedRow = -1;
	for(int32 RowIndex : RowIndices)
	{
		MaxSelectedRow = FMath::Max(RowIndex, MaxSelectedRow);
	}

	return MaxSelectedRow < Chooser->ResultsStructs.Num() - 1;
}

bool FChooserTableViewModel::IsRandomizeColumn(int32 ColumnIndex)
{
	UChooserTable* Chooser = GetChooser();
    
	if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
	{
		if (const FChooserColumnBase* Column = Chooser->ColumnsStructs[ColumnIndex].GetPtr<FChooserColumnBase>())
		{
			return (Column->IsRandomizeColumn());
		}
	}
	return false;
}

bool FChooserTableViewModel::CanMoveColumnLeft(int32 ColumnIndex)
{
	UChooserTable* Chooser = GetChooser();

	return !IsRandomizeColumn(ColumnIndex) && ColumnIndex > 0;
}

void FChooserTableViewModel::MoveColumnLeft(int32 ColumnIndex)
{
	if (CanMoveColumnLeft(ColumnIndex))
	{
		SelectColumn(GetChooser(), MoveColumn(ColumnIndex, ColumnIndex - 1));
	}
}

bool FChooserTableViewModel::CanMoveColumnRight(int32 ColumnIndex)
{
	UChooserTable* Chooser = GetChooser();
	
	if (Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
	{
		if (IsRandomizeColumn(ColumnIndex))
		{
			return false;
		}
		int32 NumColumns = Chooser->ColumnsStructs.Num();
		if (IsRandomizeColumn(NumColumns -1))
		{
			NumColumns--;
		}
		return (ColumnIndex < NumColumns-1);
	}
   	return false;
}

void FChooserTableViewModel::MoveColumnRight(int32 ColumnIndex)
{
	if (CanMoveColumnRight(ColumnIndex))
	{
		SelectColumn(GetChooser(), MoveColumn(ColumnIndex, ColumnIndex + 2));
	}
}

UChooserTable* DuplicateNestedChooser(UChooserTable* Chooser, UChooserTable* NewOuter)
{
	UChooserTable* RootTable = NewOuter->GetRootChooser();
	if (TObjectPtr<UObject>* FoundTable = RootTable->NestedObjects.FindByPredicate([Chooser](UObject* Object)
		{
			if (Cast<UChooserTable>(Object))
			{
				return Object->GetName() == Chooser->GetName();
			}
			return false;
		}))
	{
		// we already duplicated this table
		return Cast<UChooserTable>(*FoundTable);
	}
	
	UChooserTable* NewTable = NewObject<UChooserTable>(NewOuter, Chooser->GetFName(), RF_Transactional);
	NewTable->ResultsStructs = Chooser->ResultsStructs;
	NewTable->DisabledRows = Chooser->DisabledRows;
	
	// @todo: add fallback FallbackResult copy support (NewTable->FallbackResult = Chooser->FallbackResult)
	NewTable->RootChooser = RootTable;
	RootTable->AddNestedObject(NewTable);

	// copy all columns
	NewTable->ColumnsStructs = Chooser->ColumnsStructs;
	for (FInstancedStruct& ColumnData : NewTable->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.Initialize(NewTable);
	}

	// clear all column's cell data
	for (FInstancedStruct& ColumnData : NewTable->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.SetNumRows(0);
		Column.SetNumRows(Chooser->ResultsStructs.Num());
	}

	for (int32 RowIndex = 0; RowIndex < Chooser->ResultsStructs.Num(); RowIndex++)
	{
		for (int32 ColumnIndex = 0; ColumnIndex < NewTable->ColumnsStructs.Num(); ColumnIndex++)
		{
			FChooserColumnBase& SourceColumn = Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
			FChooserColumnBase& TargetColumn = NewTable->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
			TargetColumn.CopyRow(SourceColumn, RowIndex, RowIndex);
		}
	}

	for (FInstancedStruct& ResultData : NewTable->ResultsStructs)
	{
		if (FNestedChooser* NestedChooser = ResultData.GetMutablePtr<FNestedChooser>())
		{
			if (NestedChooser->Chooser)
			{
				NestedChooser->Chooser = DuplicateNestedChooser(NestedChooser->Chooser, NewTable);
			}
		}
	}

	return NewTable;
}

UChooserTable* FChooserTableViewModel::CopyRowsInternal(TConstArrayView<int32> RowIndices)
{
	UChooserTable* CopyData = NewObject<UChooserTable>(GetTransientPackage());

	UChooserTable* Chooser = GetChooser();

	// copy context data from root table
	CopyData->OutputObjectType = RootChooser->OutputObjectType;
	CopyData->ResultType = RootChooser->ResultType;
	CopyData->ContextData = RootChooser->ContextData;

	TArray<int32> RowIndicesCopy;
	RowIndicesCopy = RowIndices;
	RowIndicesCopy.Sort();
	
	
	if (RowIndicesCopy.Num() > 0 && RowIndicesCopy[0] == ColumnWidget_SpecialIndex_Fallback)
	{
		RowIndicesCopy.RemoveAt(0);
		CopyData->FallbackResult = Chooser->FallbackResult;
		if (FNestedChooser* CopiedNestedChooser = CopyData->FallbackResult.GetMutablePtr<FNestedChooser>())
		{
			// if the fallback result was a nested chooser, duplicate it
			CopiedNestedChooser->Chooser = DuplicateNestedChooser(CopiedNestedChooser->Chooser, CopyData);
		}
	}

	CopyData->ResultsStructs.SetNum(RowIndicesCopy.Num());
	CopyData->DisabledRows.SetNum(RowIndicesCopy.Num());

	// add the selected results and column data
	
	for (int32 RowIndex = 0; RowIndex < RowIndicesCopy.Num(); RowIndex++)
	{
		CopyData->ResultsStructs[RowIndex] = Chooser->ResultsStructs[RowIndicesCopy[RowIndex]];
		if (FNestedChooser* CopiedNestedChooser = CopyData->ResultsStructs[RowIndex].GetMutablePtr<FNestedChooser>())
		{
			if (CopiedNestedChooser->Chooser)
			{
				// if the result for this row was a nested chooser (with a valid chooser assigned), duplicate it
				CopiedNestedChooser->Chooser = DuplicateNestedChooser(CopiedNestedChooser->Chooser, CopyData);
			}
		}

		CopyData->DisabledRows[RowIndex] = Chooser->DisabledRows[RowIndicesCopy[RowIndex]];
	}

	// copy all columns
	CopyData->ColumnsStructs = Chooser->ColumnsStructs;
	for (FInstancedStruct& ColumnData : CopyData->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.Initialize(CopyData);
	}

	// clear all column's cell data
	for (FInstancedStruct& ColumnData : CopyData->ColumnsStructs)
	{
		FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
		Column.SetNumRows(0);
		Column.SetNumRows(RowIndicesCopy.Num());
	}

	for (int32 RowIndex = 0; RowIndex < RowIndicesCopy.Num(); RowIndex++)
	{
		for (int32 ColumnIndex = 0; ColumnIndex < CopyData->ColumnsStructs.Num(); ColumnIndex++)
		{
			FChooserColumnBase& SourceColumn = Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
			FChooserColumnBase& TargetColumn = CopyData->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
			TargetColumn.CopyRow(SourceColumn, RowIndicesCopy[RowIndex], RowIndex);
		}
	}
	
	return CopyData;
}

void FChooserTableViewModel::SetChooser(UChooserTable* Chooser)
{
	RootChooser = Chooser->GetRootChooser();
	CurrentChooser = Chooser;
	RefreshAll();
}

UChooserTable* FChooserTableViewModel::CopyColumnInternal(int32 ColumnIndex)
{
	UChooserTable* CopyData = NewObject<UChooserTable>(GetTransientPackage());

	UChooserTable* Chooser = GetChooser();

	// copy context data from root table
	CopyData->OutputObjectType = RootChooser->OutputObjectType;
	CopyData->ResultType = RootChooser->ResultType;
	CopyData->ContextData = RootChooser->ContextData;

	// add selected column including all the cell data, using CopyRow to properly transfer
	// per-row references (e.g. database pointers) into CopyData's NestedObjects so they
	// survive text serialization to the clipboard.
	CopyData->ColumnsStructs.Add(Chooser->ColumnsStructs[ColumnIndex]);
	FChooserColumnBase& SourceColumn = Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
	FChooserColumnBase& CopiedColumn = CopyData->ColumnsStructs.Last().GetMutable<FChooserColumnBase>();
	CopiedColumn.Initialize(CopyData);
	CopiedColumn.SetNumRows(0);
	CopiedColumn.SetNumRows(Chooser->ResultsStructs.Num());
	for (int32 RowIndex = 0; RowIndex < Chooser->ResultsStructs.Num(); RowIndex++)
	{
		CopiedColumn.CopyRow(SourceColumn, RowIndex, RowIndex);
	}
	CopiedColumn.CopyFallback(SourceColumn);

	return CopyData;
}
	
void FChooserTableViewModel::CopyColumn(int32 ColumnIndex)
{
	UChooserTable* CopyData = CopyColumnInternal(ColumnIndex);

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
	
	// Export the clipboard to text.
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context,CopyData, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, CopyData->GetOuter());
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

void FChooserTableViewModel::CopyRows(TConstArrayView<int32> RowsToCopy)
{
	UChooserTable* CopyData = CopyRowsInternal(RowsToCopy);

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
	
	// Export the clipboard to text.
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context,CopyData, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, CopyData->GetOuter());
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}
	
class FChooserClipboardFactory : public FCustomizableTextObjectFactory
{
public:
	
	FChooserClipboardFactory()
		: FCustomizableTextObjectFactory(GWarn)
		, ClipboardContent(nullptr) 
	{
	}

	UChooserTable* ClipboardContent;
	
protected:

	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UChooserTable::StaticClass()))
		{
			return true;
		}
		return false;
	}
	
	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (CreatedObject->IsA<UChooserTable>())
		{
			ClipboardContent = CastChecked<UChooserTable>(CreatedObject);

			for (FInstancedStruct& ColumnData : ClipboardContent->ColumnsStructs)
			{
				if (FChooserColumnBase* Column = ColumnData.GetMutablePtr<FChooserColumnBase>())
				{
					Column->Initialize(ClipboardContent);
				}
			}
		}
	}
};

static FString GetColumnName(FChooserColumnBase& Column)
{
	if (FChooserParameterBase* InputValue = Column.GetInputValue())
	{
		return InputValue->GetDebugName();
	}
	return FString();
}
	
void FChooserTableViewModel::PasteInternal(UChooserTable* PastedContent, int32 PasteRowIndex)
{
	UChooserTable* Chooser = GetChooser();
	Chooser->Modify();
	
	if (PastedContent->ResultsStructs.IsEmpty() && !PastedContent->FallbackResult.IsValid())
	{
		// pasting a column
		int32 InsertColumnIndex = Chooser->ColumnsStructs.Num();
		if (CurrentSelectionType == ESelectionType::Column && SelectedColumn)
		{
			InsertColumnIndex = FMath::Min(InsertColumnIndex, SelectedColumn->Column + 1);
		}
		
		if (Chooser->ColumnsStructs.Num() > 0 && Chooser->ColumnsStructs.Num() == InsertColumnIndex)
		{
			// if were inserting at the end, there is a randomize column, insert new columns before it
			if (Chooser->ColumnsStructs.Last().Get<FChooserColumnBase>().IsRandomizeColumn())
			{
				InsertColumnIndex--;
			}
		}

		int32 CurrentInsertColumnIndex = InsertColumnIndex;
		for (FInstancedStruct& ColumnData : PastedContent->ColumnsStructs)
		{
			FChooserColumnBase& PastedColumn = ColumnData.GetMutable<FChooserColumnBase>();
			Chooser->ColumnsStructs.Insert(ColumnData, CurrentInsertColumnIndex);
			FChooserColumnBase& Column = Chooser->ColumnsStructs[CurrentInsertColumnIndex].GetMutable<FChooserColumnBase>();
			Column.Initialize(Chooser);
			Column.SetNumRows(0);
			Column.SetNumRows(Chooser->ResultsStructs.Num());
			for (int32 RowIndex = 0; RowIndex < Chooser->ResultsStructs.Num(); RowIndex++)
			{
				Column.CopyRow(PastedColumn, RowIndex, RowIndex);
			}
			Column.CopyFallback(PastedColumn);
			++CurrentInsertColumnIndex;
		}

		SelectColumn(Chooser, InsertColumnIndex);
	}
	else
	{
		// pasting rows
		int32 RowsToPaste = PastedContent->ResultsStructs.Num();

		// figure out where to start inserting
		int32 InsertIndex = Chooser->ResultsStructs.Num();

		if (PasteRowIndex >=0)
		{
			InsertIndex = PasteRowIndex;
		}
		else
		{
			if (SelectedRows.Num() > 0)
			{
				InsertIndex = SelectedRows[0]->Row;
				for(int32 SelectedRowIndex = 1; SelectedRowIndex < SelectedRows.Num(); SelectedRowIndex ++)
				{
					InsertIndex = FMath::Max(InsertIndex, SelectedRows[SelectedRowIndex]->Row);
				}
				if (InsertIndex == ColumnWidget_SpecialIndex_Fallback)
				{
					// if the only row selected was the fallback, reset insert index to the last row
					InsertIndex = Chooser->ResultsStructs.Num();
				}
				else
				{
					InsertIndex++;
				}
			}
		}

		if (PastedContent->ResultsStructs.Num() > 0)
		{
			Chooser->ResultsStructs.Insert(PastedContent->ResultsStructs, InsertIndex);
			Chooser->DisabledRows.Insert(PastedContent->DisabledRows, InsertIndex);

			// Make sure each column has the same number of row datas as there are results
			for(FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
			{
				FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
				Column.InsertRows(InsertIndex, RowsToPaste);
			}
		}
		if (PastedContent->FallbackResult.IsValid())
		{
			// paste fallback result if copy data has one
			Chooser->FallbackResult = PastedContent->FallbackResult;
			if (FNestedChooser* NestedChooser = Chooser->FallbackResult.GetMutablePtr<FNestedChooser>())
			{
				// duplicate the nested chooser if the fallback result refers to a nested chooser
				NestedChooser->Chooser = DuplicateNestedChooser(NestedChooser->Chooser, Chooser);
			}
		}
		
		if (!PastedContent->NestedObjects.IsEmpty())
		{
			// if there were nested choosers in the copy buffer we have to remap or paste them here
			for (int32 ResultIndex = InsertIndex; ResultIndex < PastedContent->ResultsStructs.Num() + InsertIndex; ResultIndex++)
			{
				if (FNestedChooser* NestedChooser = Chooser->ResultsStructs[ResultIndex].GetMutablePtr<FNestedChooser>())
				{
					NestedChooser->Chooser = DuplicateNestedChooser(NestedChooser->Chooser, Chooser);
				}
			}
		}

		// try to also paste column data from columns in the paste buffer which match the columns in the current chooser
		// -- matching by column type and input value name

		// keep track of target columns that have already been matched, to avoid matching multiple source columns with the same target column
		TArray<bool> MatchedTargetColumns;
		MatchedTargetColumns.SetNum(Chooser->ColumnsStructs.Num());

		// keep track of which source columns were matched, so we can add new columns for the unmatched ones after
		TArray<bool> MatchedSourceColumns;
		MatchedSourceColumns.SetNum(PastedContent->ColumnsStructs.Num());
		
		for(int32 SourceColumnIndex = 0; SourceColumnIndex < PastedContent->ColumnsStructs.Num(); SourceColumnIndex++)
		{
			FInstancedStruct& PastedColumnData = PastedContent->ColumnsStructs[SourceColumnIndex];
			FChooserColumnBase& PastedColumn = PastedColumnData.GetMutable<FChooserColumnBase>();
			FString PastedColumnName = GetColumnName(PastedColumn);
			for(int32 TargetColumnIndex = 0; TargetColumnIndex < Chooser->ColumnsStructs.Num(); TargetColumnIndex++)
			{
				if (!MatchedTargetColumns[TargetColumnIndex])
				{
					FInstancedStruct& ColumnData = Chooser->ColumnsStructs[TargetColumnIndex];
					if (ColumnData.GetScriptStruct() == PastedColumnData.GetScriptStruct())
					{
						FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
						FString ColumnName = GetColumnName(Column);

						if (ColumnName == PastedColumnName)
						{
							MatchedTargetColumns[TargetColumnIndex] = true;
							MatchedSourceColumns[SourceColumnIndex] = true;

							// found a match, copy the data over
							for (int32 i = 0; i < RowsToPaste; i++)
							{
								Column.CopyRow(PastedColumn, i, InsertIndex + i);
							}

							if (PastedContent->FallbackResult.IsValid())
							{
								// if the fallback row was copied, paste the fallback data for columns
								Column.CopyFallback(PastedColumn);
							}
							break;
						}
					}
				}
			}
		}
		
		// add new columns for any source columns that were unmatched
		
		int32 InsertColumnIndex = Chooser->ColumnsStructs.Num();
		if (Chooser->ColumnsStructs.Num() > 0)
		{
			// if there is a randomize column, insert new columns before it
			if (Chooser->ColumnsStructs.Last().Get<FChooserColumnBase>().IsRandomizeColumn())
			{
				InsertColumnIndex--;
			}
		}

		for(int32 SourceColumnIndex = 0; SourceColumnIndex < PastedContent->ColumnsStructs.Num(); SourceColumnIndex++)
		{
			if (!MatchedSourceColumns[SourceColumnIndex])
			{
				FInstancedStruct& PastedColumnData = PastedContent->ColumnsStructs[SourceColumnIndex];
				FChooserColumnBase& PastedColumn = PastedColumnData.GetMutable<FChooserColumnBase>();
				// if we couldn't find a match, paste a new column
				Chooser->ColumnsStructs.Insert(PastedColumnData, InsertColumnIndex);
				FChooserColumnBase& Column = Chooser->ColumnsStructs[InsertColumnIndex].GetMutable<FChooserColumnBase>();
				Column.Initialize(Chooser);
				InsertColumnIndex++;
				Column.SetNumRows(0);
				Column.SetNumRows(Chooser->ResultsStructs.Num());
				for (int32 i = 0; i < RowsToPaste; i++)
				{
					Column.CopyRow(PastedColumn, i, InsertIndex + i);
				}
			}
		}

		ClearSelectedRows();
		RefreshAll();

		TArray<int32, TInlineAllocator<256>> SelectedIndices;
		SelectedIndices.Reserve(RowsToPaste+1);
		
		// select the inserted rows
		for (int32 i = 0; i < RowsToPaste; i++)
		{
			SelectedIndices.Add(InsertIndex + i);
		}
		if(PastedContent->FallbackResult.IsValid())
		{
			SelectedIndices.Add(ColumnWidget_SpecialIndex_Fallback);
		}

		SelectRows(SelectedIndices);
	}
}

void FChooserTableViewModel::Paste()
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	FChooserClipboardFactory Factory;

	UChooserTable* Chooser = GetChooser();

	if (Factory.CanCreateObjectsFromText(ClipboardText))
	{
		Factory.ProcessBuffer((UObject*)GetTransientPackage(), RF_Transactional, ClipboardText);
		if (UChooserTable* PastedContent = Factory.ClipboardContent)
		{
			FScopedTransaction Transaction(LOCTEXT("Paste Chooser Data", "Paste Chooser Data"));
			PasteInternal(PastedContent);
			RefreshAll();
		}
	}
}

	
bool FChooserTableViewModel::CanPaste() const
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	FChooserClipboardFactory Factory;
	return Factory.CanCreateObjectsFromText(ClipboardText); 
}

// Selection stuff to move to widget
	
void FChooserTableViewModel::SelectColumn(UChooserTable* ChooserEditor, int32 Index)
{
	ClearSelectedRows();
	
	UChooserTable* Chooser = GetChooser();
   	if (Chooser->ColumnsStructs.IsValidIndex(Index))
   	{
   		if (SelectedColumn == nullptr)
   		{
   			SelectedColumn = TStrongObjectPtr(NewObject<UChooserColumnDetails>(RootChooser, NAME_None, RF_Transactional | RF_Transient));
   		}
   
   		SelectedColumn->Chooser = Chooser;
   		SelectedColumn->Column = Index;

   		ShowDetailsDelegate.ExecuteIfBound({SelectedColumn.Get()});
   		
   		CurrentSelectionType = ESelectionType::Column;
   	}
}
	
void FChooserTableViewModel::ClearSelectedColumn()
{
	UChooserTable* Chooser = GetChooser();
	if (CurrentSelectionType == ESelectionType::Column && SelectedColumn)
	{
		SelectedColumn->Column = INDEX_NONE;
	}
}
	
void FChooserTableViewModel::SelectRootProperties()
{
	ClearSelectedColumn();
	ClearSelectedRows();
	
	ShowDetailsDelegate.ExecuteIfBound( { GetRootChooser() } );
	
	CurrentSelectionType = ESelectionType::Root;
}

void FChooserTableViewModel::SetSelectedRows(const TConstArrayView<int32>& Rows)
{
	ClearSelectedColumn();
	SetCurrentSelectionType(FChooserTableViewModel::ESelectionType::Rows);
	SelectedRowIndices = Rows;
	
	UChooserTable* Chooser = GetChooser();
	SelectedRowIndices.RemoveAll([Chooser](int32 Index)
	{
		// remove any special indices except the fallback row
		return !(Chooser->ResultsStructs.IsValidIndex(Index) || Index == ColumnWidget_SpecialIndex_Fallback);
	});
	
	RefreshRowSelectionDetails();
}
	
void FChooserTableViewModel::SelectRows(const TConstArrayView<int32>& Rows)
{
	SelectRowsDelegate.ExecuteIfBound(Rows);
}

void FChooserTableViewModel::RefreshAssetRegistry()
{
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		// this causes the UAF workspace outliner to refresh
		AssetRegistry->AssetUpdateTags(RootChooser, EAssetRegistryTagsCaller::Fast);
	}	
}

void FChooserTableViewModel::ClearSelectedRows()
{
	SelectRows({});
}

bool FChooserTableViewModel::IsRowSelected(int32 RowIndex)
{
	for(auto& SelectedRow:SelectedRows)
 	{
 		if (SelectedRow->Row == RowIndex)
 		{
 			return true;
 		}
 	}
	return false;
}
	
bool FChooserTableViewModel::IsColumnSelected(int32 ColumnIndex)
{
	return (CurrentSelectionType == ESelectionType::Column && SelectedColumn && SelectedColumn->Column == ColumnIndex);
}

void FChooserTableViewModel::MakeDebugTargetMenu(UToolMenu* InToolMenu)
{
	static FName SectionName = "Select Debug Target";
	InToolMenu->bSearchable = true;

	InToolMenu->AddMenuEntry(
		SectionName,
		FToolMenuEntry::InitMenuEntry(
			"None",
			LOCTEXT("None", "None"),
			LOCTEXT("None Tooltip", "Clear selected debug target"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
					{
						UChooserTable* Chooser = GetRootChooser();
						Chooser->ResetDebugTarget();
						if (Chooser->GetEnableDebugTesting())
						{
							Chooser->SetEnableDebugTesting(false);
							Chooser->SetDebugTestValuesValid(false);
							RefreshAll();
						}
					}),
				FCanExecuteAction()
			)
		));

	InToolMenu->AddMenuEntry(
		SectionName,
		FToolMenuEntry::InitMenuEntry(
			"Manual",
			LOCTEXT("Manual Testing", "Manual Testing"),
			LOCTEXT("Manual Tooltip", "Test the chooser by manually entering values for each column"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
					{
						UChooserTable* Chooser = GetRootChooser();
						Chooser->ResetDebugTarget();
						if (!Chooser->GetEnableDebugTesting())
						{
							Chooser->SetEnableDebugTesting(true);
							Chooser->SetDebugTestValuesValid(true);
							RefreshAll();
						}
					}),
				FCanExecuteAction()
			)
		));

	const UChooserTable* Chooser = GetRootChooser();

	Chooser->IterateRecentContextObjects([this, InToolMenu](const FString& ObjectName)
		{
			InToolMenu->AddMenuEntry(
				SectionName,
				FToolMenuEntry::InitMenuEntry(
					FName(ObjectName),
					FText::FromString(ObjectName),
					LOCTEXT("Select Object ToolTip", "Select this object as the debug target"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, ObjectName]()
							{
								UChooserTable* Chooser = GetRootChooser();
								Chooser->SetDebugTarget(ObjectName);
								Chooser->SetDebugTestValuesValid(false);
								if (!Chooser->GetEnableDebugTesting())
								{
									Chooser->SetEnableDebugTesting(true);
									RefreshAll();
								}
							}),
						FCanExecuteAction()
					)
				));
		}
	);

}

	
}

#undef LOCTEXT_NAMESPACE

