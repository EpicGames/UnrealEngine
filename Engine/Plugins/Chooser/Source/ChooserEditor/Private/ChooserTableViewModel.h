// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chooser.h"
#include "EditorUndoClient.h"
#include "PropertyEditorDelegates.h"
#include "Containers/RingBuffer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Views/SListView.h"
#include "ChooserDetails.h"
#include "IChooserTableViewModel.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserTableViewModel.generated.h"

class SComboButton;
class SPositiveActionButton;
class SEditableText;

// Class used for chooser editor details customization
UCLASS()
class UChooserColumnDetails : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Instanced, Category="Hidden")
	TObjectPtr<UChooserTable> Chooser;
	int32 Column = -1;
};


namespace UE::ChooserEditor
{
	class SChooserTableWidget;
	class SNestedChooserTree;
	struct FChooserTableRow;

	DECLARE_DELEGATE(FRefreshTable);
	DECLARE_DELEGATE_OneParam(FSelectRows, TConstArrayView<int32>);

	class FChooserTableViewModel : public IChooserTableViewModel, public IChooserTableWidgetInterface, public FSelfRegisteringEditorUndoClient
	{
	public:
		virtual ~FChooserTableViewModel() override;
		FChooserTableViewModel(UChooserTable* Chooser);

		virtual void OpenObject(UObject* Object) override
		{
			OpenObjectDelegate.ExecuteIfBound(Object);
		}

		// IChooserTableViewModel interface
		virtual void SetOpenObjectDelegate(FOpenObject InOpenObjectDelegate) override
		{
			OpenObjectDelegate = InOpenObjectDelegate;
		}
		
		virtual void SetShowDetailsDelegate(FShowDetails InShowDetailsDelegate) override
		{
			ShowDetailsDelegate = InShowDetailsDelegate;
		}

		virtual void RegisterMenus(TSharedPtr<FUICommandList> CommandList) override;

		virtual const UChooserTable* GetRootChooser() const override { return RootChooser; }
		virtual UChooserTable* GetRootChooser() override { return RootChooser; }
		
		virtual UChooserTable* GetChooser() override { return CurrentChooser; }
		virtual const UChooserTable* GetChooser() const override { return CurrentChooser; }

		virtual void RefreshAll() override;

		virtual void SetChooser(UChooserTable* Chooser) override;

		virtual void AutoPopulateAll() override;

		virtual void SelectRootProperties() override;
		
		void MakeDebugTargetMenu(UToolMenu* InToolMenu);
		
		/** Begin FNotifyHook Interface */
		virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;
		virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

		int32 GetNumRows() const;

		bool AreRowsDisabled(TConstArrayView<int32> RowIndices) const;
		bool IsColumnDisabled(int32 ColumnIndex) const;
		void ToggleDisableRows(TConstArrayView<int32> RowIndices);
		void ToggleDisableColumn(int32 ColumnIndex);
		void DuplicateColumn(int32 ColumnIndex);
		void DuplicateRows(TConstArrayView<int32> RowIndices);
		bool HasFallbackSelected();
		
		bool IsRandomizeColumn(int32 ColumnIndex);

		bool CanMoveRowsUp(TConstArrayView<int32> RowIndices);
		bool CanMoveRowsDown(TConstArrayView<int32> RowIndices);
		bool CanMoveColumnLeft(int32 ColumnIndex);
		void MoveColumnLeft(int32 ColumnIndex);
		bool CanMoveColumnRight(int32 ColumnIndex);
		void MoveColumnRight(int32 ColumnIndex);
		
		void CopyColumn(int32 ColumnIndex);
		void CopyRows(TConstArrayView<int> RowIndices);
				
		void RemoveDisabledData();

		bool CanAutoPopulateColumn(int32 ColumnIndex) const;
		bool CanAutoPopulateRows() const;
		void AutoPopulateColumn(int32 ColumnIndex);
		void AutoPopulateRows(TConstArrayView<int> RowIndices);
		void AutoPopulateRow(int32 Index);
		
		void ClearSelectedColumn();
		void DeleteColumn(int32 Index);
		void AddColumn(const UScriptStruct* ColumnType);

		
		void DeleteRows(TConstArrayView<int32> RowsToDelete);
		int32 DeleteRowsInternal(TConstArrayView<int32> RowsToDelete, int32 RowIndexToRemember = 0);
		
		void MoveRows(TConstArrayView<int32> RowIndices, int32 TargetIndex);
		int32 MoveColumn(int32 SourceIndex, int32 TargetIndex);
		void Paste();
		bool CanPaste() const;

		enum class ESelectionType
		{
			Root, Rows, Column
		};
		
		void RefreshRowSelectionDetails();
		ESelectionType GetCurrentSelectionType() const { return CurrentSelectionType; }
		void SetCurrentSelectionType(ESelectionType SelectionType) { CurrentSelectionType = SelectionType; }

		void RefreshAssetRegistry(); // update nested chooser info in asset registry

		void ClearSelectedRows();
		void SelectRows(const TConstArrayView<int32>& Rows);		// called internally to change row selection after operations
		void SelectColumn(UChooserTable* Chooser, int32 Index);
		bool IsRowSelected(int32 RowIndex);
		bool IsColumnSelected(int32 ColumnIndex);
		bool HasSelection() const;
		bool HasRowsSelected() const;
		bool HasColumnSelected() const;

		void SetSelectedRows(const TConstArrayView<int32>& Rows); // called by widget when list view selection changes
	
		int32 GetSelectedColumn()
		{
			if (SelectedColumn)
			{
				return SelectedColumn->Column;
			}
			return INDEX_NONE;
		}
		
		TConstArrayView<int32> GetSelectedRows() const
		{
			return SelectedRowIndices;
		}
		
		ESelectionType CurrentSelectionType = ESelectionType::Root;

		TArray<int32> SelectedRowIndices;
		TStrongObjectPtr<UChooserColumnDetails> SelectedColumn;
		TArray<TObjectPtr<UChooserRowDetails>> SelectedRows;
		// cache of objects to reuse for SelectedRows
		TArray<TStrongObjectPtr<UChooserRowDetails>> SelectedRowsCache;
		
		FOpenObject OpenObjectDelegate;
		FShowDetails ShowDetailsDelegate;
		FRefreshTable RefreshTableColumnsDelegate;
		FRefreshTable RefreshTableRowsDelegate;
		FSelectRows SelectRowsDelegate;
		
		UChooserTable* CopyColumnInternal(int32 ColumnIndex);
		UChooserTable* CopyRowsInternal(TConstArrayView<int32> RowIndices);
		void PasteInternal(UChooserTable* PasteObject, int32 PasteRowIndex=-1);

		void UpdateTableRows();
		void UpdateTableColumns();

		void QueueRowTransaction(UChooserRowDetails* RowDetails);
		void OnObjectsTransacted(UObject* Object, const FTransactionObjectEvent& Event);

		/** FEditorUndoClient Interface */
		virtual bool MatchesContext( const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts ) const override;
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;
		
	private:
		void AutoPopulateColumn(FChooserColumnBase& Column);
		
		bool HandleTicker(float DeltaTime);

		FTSTicker::FDelegateHandle TickDelegateHandle;
		
		// the root asset for the chooser editor
		UChooserTable* RootChooser = nullptr;

		// the current (possibly nested) chooser table being edited
		UChooserTable* CurrentChooser = nullptr;

		TArray<TWeakObjectPtr<UChooserRowDetails>> QueuedRowTransactions;
	};
	
}