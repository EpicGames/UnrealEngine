// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTraceModule.h"
#include "ChooserProvider.h"
#include "ChooserAnalyzer.h"

FName FChooserTraceModule::ModuleName("Chooser");

void FChooserTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("Chooser");
}

void FChooserTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FChooserProvider> ChooserProvider = MakeShared<FChooserProvider>(InSession);
	InSession.AddProvider(FChooserProvider::ProviderName, ChooserProvider);

	InSession.AddAnalyzer(new FChooserAnalyzer(InSession, *ChooserProvider));
}

void FChooserTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("Chooser"));
}

void FChooserTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

