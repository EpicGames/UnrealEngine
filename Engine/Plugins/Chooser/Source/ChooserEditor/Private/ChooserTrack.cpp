// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChooserTrack.h"

#include "Chooser.h"
#include "ChooserProvider.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "GameFramework/Actor.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "ChooserTrack"

namespace UE::ChooserEditor
{

FChooserTrack::FChooserTrack(uint64 InObjectId, uint64 InChooserId) :
	ObjectId(InObjectId),
	ChooserId(InChooserId)
{
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
	
	const IGameplayProvider* GameplayProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<IGameplayProvider>("GameplayProvider");
	FString ChooserName;
	FString ChooserPathName;
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		const FObjectInfo& ChooserInfo = GameplayProvider->GetObjectInfo(InChooserId);
		ChooserName = ChooserInfo.Name;
		ChooserPathName = ChooserInfo.PathName;
	}
	TrackName = FText::FromString(ChooserName);
	ChooserTable = FindObject<UChooserTable>(nullptr, *ChooserPathName);

}

FChooserTrack::~FChooserTrack()
{
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FChooserTrack::GetEventData() const
{
	if (!EventData.IsValid())
	{
		EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	}
	
	EventUpdateRequested++;
	
	return EventData;
}
	
static FLinearColor MakeNotifyColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

TSharedPtr<SWidget> FChooserTrack::GetDetailsViewInternal() 
{
	return nullptr;
}

bool FChooserTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> RecordingTimeRange = RewindDebugger->GetCurrentViewRange();
	double StartTime = RecordingTimeRange.GetLowerBoundValue();
	double EndTime = RecordingTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FChooserProvider* ChooserProvider = AnalysisSession->ReadProvider<FChooserProvider>(FChooserProvider::ProviderName);
	
	if(EventUpdateRequested > 10 && ChooserProvider)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FChooserTrack::UpdateEventPointsInternal);
		EventUpdateRequested = 0;
		
		EventData->EventPoints.SetNum(0,EAllowShrinking::No);
		EventData->EventWindows.SetNum(0);

		TraceServices::FProviderReadScopeLock ProviderReadScope(*ChooserProvider);
		ChooserProvider->ReadChooserEvaluationTimeline(ObjectId, [this, StartTime, EndTime](const FChooserProvider::ChooserEvaluationTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime](double InStartTime, double InEndTime, uint32 InDepth, const FChooserEvaluationData& ChooserEvaluationData)
			{
				if (ChooserEvaluationData.ChooserId == ChooserId)
				{
					EventData->EventPoints.Add({InStartTime, FText(), FText(), FLinearColor::White});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	EventUpdateRequested++;

	bool bChanged = false;
	return bChanged;
	
}

bool FChooserTrack::HandleDoubleClickInternal()
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
		if (!GameplayProvider)
		{
			return false;
		}

		FString AssetPathName;
		FString DebugName;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(GetChooserId());
			AssetPathName = AssetInfo.PathName;

			// attach chooser table editor debugging
			if (ChooserTable)
			{
				const FObjectInfo& OwnerObjectInfo  = GameplayProvider->GetObjectInfo(ObjectId);
				DebugName = FString(OwnerObjectInfo.Name);
				if (const FObjectInfo* ActorInfo = RewindDebugger->FindTypedOuterInfo<AActor>(GameplayProvider, ObjectId))
				{
					DebugName += " in " + FString(ActorInfo->Name);
				}

				if (const FWorldInfo* WorldInfo = GameplayProvider->FindWorldInfoFromObject(ObjectId))
				{
					const FObjectInfo& WorldObjectInfo = GameplayProvider->GetObjectInfo(WorldInfo->Id);

					if (WorldInfo->NetMode == FWorldInfo::ENetMode::DedicatedServer)
					{
						DebugName += " (Server)";
					}
					else if (WorldInfo->NetMode == FWorldInfo::ENetMode::Client)
					{
						DebugName += FString(" (Client ") + FString::FromInt(WorldInfo->PIEInstanceId) + ")";
					}
				}
			}
		}

		if (ChooserTable)
		{
			UChooserTable* RootChooser = ChooserTable->GetRootChooser();
			RootChooser->SetDebugTarget(DebugName);
			RootChooser->SetEnableDebugTesting(true);

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ChooserTable);
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetPathName);
		}

		return true;
	}
	return false;
}

TSharedPtr<SWidget> FChoosersTrack::GetDetailsViewInternal() 
{
	return nullptr;
}

TSharedPtr<SWidget> FChooserTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FChooserTrack::GetEventData);
}

static const FName ChoosersName("Choosers");

FName FChoosersTrackCreator::GetTargetTypeNameInternal() const
{
	return UObject::StaticClass()->GetFName();
}
	
FName FChoosersTrackCreator::GetNameInternal() const
{
	return ChoosersName;
}

void FChoosersTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ChoosersName, LOCTEXT("Chooser", "Choosers")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FChoosersTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FChoosersTrack>(InObjectId.GetMainId());
}

		
FChoosersTrack::FChoosersTrack(uint64 InObjectId) :
	ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}


bool FChoosersTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChoosersTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> RecordingTimeRange = RewindDebugger->GetCurrentViewRange();
	double StartTime = RecordingTimeRange.GetLowerBoundValue();
	double EndTime = RecordingTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FChooserProvider* ChooserProvider = AnalysisSession->ReadProvider<FChooserProvider>(FChooserProvider::ProviderName);
	
	bool bChanged = false;
	
	if(ChooserProvider)
	{
		TArray<uint64> UniqueTrackIds;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*ChooserProvider);

			ChooserProvider->ReadChooserEvaluationTimeline(ObjectId, [this, StartTime, EndTime, &UniqueTrackIds](const FChooserProvider::ChooserEvaluationTimeline& InTimeline)
			{
				InTimeline.EnumerateEvents(StartTime, EndTime, [this, &UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FChooserEvaluationData& ChooserEvaluationData)
				{
					UniqueTrackIds.AddUnique(ChooserEvaluationData.ChooserId);
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		UniqueTrackIds.StableSort();
		const int32 TrackCount = UniqueTrackIds.Num();

		if (Children.Num() != TrackCount)
			bChanged = true;

		Children.SetNum(UniqueTrackIds.Num());
		for(int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || !(Children[i].Get()->GetChooserId() == UniqueTrackIds[i]))
			{
				Children[i] = MakeShared<FChooserTrack>(ObjectId, UniqueTrackIds[i]);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}
	
	return bChanged;
}

TConstArrayView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> FChoosersTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FChoosersTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChoosersTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	
	bool bHasData = false;
	if (const FChooserProvider* ChooserProvider = AnalysisSession->ReadProvider<FChooserProvider>(FChooserProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*ChooserProvider);
		ChooserProvider->ReadChooserEvaluationTimeline(InObjectId.GetMainId(), [&bHasData](const FChooserProvider::ChooserEvaluationTimeline& InTimeline)
	 	{
	 		bHasData = true;
	 	});
	}
	return bHasData;
}

	
}

#undef LOCTEXT_NAMESPACE