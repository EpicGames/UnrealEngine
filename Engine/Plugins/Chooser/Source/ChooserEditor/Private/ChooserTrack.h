// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"

class UChooserTable;
	
namespace UE::ChooserEditor
{

class FChooserTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FChooserTrack(uint64 InObjectId, uint64 InChooserId);
	virtual ~FChooserTrack();
	
	uint64 GetChooserId() { return ChooserId; }
private:
	virtual bool UpdateInternal() override;
	TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FText GetDisplayNameInternal() const override { return TrackName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	// virtual FName GetNameInternal() const override { return Category; }
	virtual bool HandleDoubleClickInternal() override;

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;
	
	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int EventUpdateRequested = 0;

	UChooserTable* ChooserTable = 0;
	uint64 ObjectId;
	uint64 ChooserId;
	FSlateIcon Icon;
	FText TrackName;
};



// Parent track 
class FChoosersTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FChoosersTrack(uint64 InObjectId);
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "Choosers"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "ChooserTrackName", "Chooser Evaluation"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	FSlateIcon Icon;
	uint64 ObjectId;

	TArray<TSharedPtr<FChooserTrack>> Children;
};

#if WITH_ENGINE
class FChoosersTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}