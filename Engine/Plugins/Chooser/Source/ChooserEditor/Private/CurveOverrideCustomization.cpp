// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveOverrideCustomization.h"

#include "AnimNode_ChooserPlayer.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "SClassViewer.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "CurveOverrideCustomization"

namespace UE::ChooserEditor
{

	
void RebuildHash(TSharedPtr<IPropertyHandle> CurveOverrideListProperty)
{
	if (CurveOverrideListProperty)
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(CurveOverrideListProperty->GetProperty()))
		{
			if (StructProperty->Struct == FAnimCurveOverrideList::StaticStruct())
			{
				CurveOverrideListProperty->EnumerateRawData([](void* RawData, const int32 DataIndex, const int32 NumDatas)
				{
					FAnimCurveOverrideList* AnimCurveOverrideList = static_cast<FAnimCurveOverrideList*>(RawData);
					AnimCurveOverrideList->ComputeHash();
					return true;
				});
			}
		}
		else
		{
			if (TSharedPtr<IPropertyHandleStruct> StructPropertyHandle = CurveOverrideListProperty->AsStruct())
			{
				if (StructPropertyHandle->GetStructData()->GetStruct() == FAnimCurveOverrideList::StaticStruct())
				{
					CurveOverrideListProperty->EnumerateRawData([](void* RawData, const int32 DataIndex, const int32 NumDatas)
					{
						FAnimCurveOverrideList* AnimCurveOverrideList = static_cast<FAnimCurveOverrideList*>(RawData);
						AnimCurveOverrideList->ComputeHash();
						return true;
					});
				}
			}
		}
	}
}
	
	
void FCurveOverrideListCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	auto ValuesPropertyHandle = PropertyHandle->GetChildHandle("Values");
	
	HeaderRow
    	.NameContent()
    	[
    		PropertyHandle->CreatePropertyNameWidget()
    	]
    	.ValueContent()
		[
			ValuesPropertyHandle->CreatePropertyValueWidget()
		];
}


/** 
 * Node builder for Array children.
 * Expects Property handle to be an Array
 */
class FArrayDataDetails : public IDetailCustomNodeBuilder, public TSharedFromThis<FArrayDataDetails>
{
public:
	FArrayDataDetails(TSharedPtr<IPropertyHandle> InArrayProperty)
	{
		ArrayProperty = InArrayProperty;
	}
	virtual ~FArrayDataDetails() override { };

	//~ Begin IDetailCustomNodeBuilder interface
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override
	{
		OnRegenerateChildren = InOnRegenerateChildren;
	};
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {};
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		ArrayProperty->GetNumChildren(CachedNum);
		for(uint32 i=0;i<CachedNum;i++)
		{
			if (auto ChildProperty = ArrayProperty->GetChildHandle(i))
			{
				ChildrenBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	};
	
	virtual void Tick(float DeltaTime) override
	{
		// If the array num changes regenerate
		uint32 NumChildren = 0;
		ArrayProperty->GetNumChildren(NumChildren);
		if (ArrayProperty && NumChildren != CachedNum)
		{
			OnRegenerateChildren.ExecuteIfBound();
		}	
	}
	
	virtual bool RequiresTick() const override { return true; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override
	{
		static const FName Name("InstancedStructDataDetails");
		return Name;
	}
	//~ End IDetailCustomNodeBuilder interface
private:
	/** Handle to the array property being edited */
	TSharedPtr<IPropertyHandle> ArrayProperty;

	/** Delegate that can be used to refresh the child rows of the current struct (eg, when changing struct type) */
	FSimpleDelegate OnRegenerateChildren;

	uint32 CachedNum = 0;
};

	
	
void FCurveOverrideListCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (auto ValuesPropertyHandle = PropertyHandle->GetChildHandle("Values"))
	{
		ChildBuilder.AddCustomBuilder(MakeShared<FArrayDataDetails>(ValuesPropertyHandle));
	}
}

void FCurveOverrideCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	auto NamePropertyHandle = PropertyHandle->GetChildHandle("CurveName");
	auto ValuePropertyHandle = PropertyHandle->GetChildHandle("CurveValue");

	HeaderRow
		.NameContent().HAlign(EHorizontalAlignment::HAlign_Right).MinDesiredWidth(300)
		[
			SNew(SEditableTextBox)
			.Text_Lambda([NamePropertyHandle]
			{
				FName Name;
				NamePropertyHandle->GetValue(Name);
				return FText::FromName(Name);
			})
			.OnTextCommitted_Lambda([NamePropertyHandle](FText NewValue, ETextCommit::Type CommitType)
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("Set Curve Override Name", "Set Curve Override Name"));
				NamePropertyHandle->NotifyPreChange();
				NamePropertyHandle->SetValue(FName(NewValue.ToString()));
				NamePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

				if (TSharedPtr<IPropertyHandle> ParentStructProperty = NamePropertyHandle->GetParentHandle())
				{
					if (TSharedPtr<IPropertyHandle> ParentArrayProperty = ParentStructProperty->GetParentHandle())
					{
						if (TSharedPtr<IPropertyHandle> CurveOverrideListProperty = ParentArrayProperty->GetParentHandle())
						{
							RebuildHash(CurveOverrideListProperty);
						}
					}
				}
			})
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Value_Lambda([ValuePropertyHandle]()
			{
				float Value;
				ValuePropertyHandle->GetValue(Value);
				return Value;
			})
			.OnValueCommitted_Lambda([ValuePropertyHandle](float NewValue, ETextCommit::Type CommitType)
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("Set Curve Override Value", "Set Curve Override Value"));
				ValuePropertyHandle->NotifyPreChange();
				ValuePropertyHandle->SetValue(NewValue);
				ValuePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				
				if (TSharedPtr<IPropertyHandle> ParentStructProperty = ValuePropertyHandle->GetParentHandle())
				{
					if (TSharedPtr<IPropertyHandle> ParentArrayProperty = ParentStructProperty->GetParentHandle())
					{
						if (TSharedPtr<IPropertyHandle> CurveOverrideListProperty = ParentArrayProperty->GetParentHandle())
						{
							RebuildHash(CurveOverrideListProperty);
						}
					}
				}
			})
		];
}
	
void FCurveOverrideCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE