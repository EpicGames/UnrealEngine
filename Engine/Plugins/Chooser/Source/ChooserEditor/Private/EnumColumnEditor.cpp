// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnumColumnEditor.h"
#include "ChooserColumnHeader.h"
#include "ChooserTableEditor.h"
#include "DetailLayoutBuilder.h"
#include "EnumColumn.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "OutputEnumColumn.h"
#include "ScopedTransaction.h"
#include "SPropertyAccessChainWidget.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "EnumColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> SEnumCell::GenerateEnumMenu() const
{
	if (const UEnum* EnumSource = Enum.Get())
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		for (int32 EnumIndex = 0; EnumIndex < EnumSource->NumEnums() - 1; ++EnumIndex)
		{
			const bool bIsHidden = EnumSource->HasMetaData(TEXT("Hidden"), EnumIndex);
			if (!bIsHidden)
			{
				MenuBuilder.AddMenuEntry(EnumSource->GetDisplayNameTextByIndex(EnumIndex),
					FText()/*todo tooltip*/,
					FSlateIcon(),
					FUIAction(
					FExecuteAction::CreateLambda([this, MenuEntryEnumValue = EnumSource->GetValueByIndex(EnumIndex)]() 
					{
						OnValueSet.ExecuteIfBound(MenuEntryEnumValue);
					})	
					));
			}
		}
		return MenuBuilder.MakeWidget();
	}
	return SNullWidget::NullWidget;
}
	
TSharedRef<SWidget> SEnumCell::CreateEnumComboBox()
{
	return SNew(SComboButton)
		.IsEnabled_Lambda([this](){ return IsEnabled(); } )
		.OnGetMenuContent(this, &SEnumCell::GenerateEnumMenu)
		.VAlign(VAlign_Center)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([this]()
			{
				if (const UEnum* EnumSource = Enum.Get())
				{
					return EnumSource->GetDisplayNameTextByValue(EnumValue.Get());
				}
				return FText();
			})
		];
}

void SEnumCell::Construct( const FArguments& InArgs)
{
	SetEnabled(InArgs._IsEnabled);
	
	Enum = InArgs._Enum;
	EnumValue = InArgs._EnumValue;
	OnValueSet = InArgs._OnValueSet;

	ChildSlot
	[
		CreateEnumComboBox()
	];
}


TSharedRef<SWidget> CreateEnumColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FEnumColumn* EnumColumn = static_cast<FEnumColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Enum Value Tooltip", "Enum Value: cells pass if the cell value is equal to the column input value");
		const FText ColumnName = LOCTEXT("Enum Value","Enum Value");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SEnumCell)
								.Enum_Lambda([EnumColumn]() { return EnumColumn->GetEnum(); })
								.OnValueSet_Lambda([EnumColumn](int Value) { EnumColumn->TestValue = Value; })
								.EnumValue_Lambda([EnumColumn]() { return EnumColumn->TestValue; })
								.IsEnabled_Lambda([Chooser] { return !Chooser->HasDebugTarget(); });
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget,
			FChooserWidgetValueChanged::CreateLambda([EnumColumn]()
			{
				EnumColumn->EnumChanged(EnumColumn->InputValue.Get<FChooserParameterEnumBase>().GetEnum());
			}));
	}

	// create cell widget
	
	return SNew(SHorizontalBox)
    		+ SHorizontalBox::Slot().AutoWidth()
    		[
    			SNew(SBox).WidthOverride(Row < 0 ? 0 : 55)
    			[
    				SNew(SButton).ButtonStyle(FAppStyle::Get(),"FlatButton").TextStyle(FAppStyle::Get(),"RichTextBlock.Bold").HAlign(HAlign_Center)
    				.Visibility(Row < 0 ? EVisibility::Hidden : EVisibility::Visible)
					.Text_Lambda([EnumColumn, Row]()
					{
						if (EnumColumn->RowValues.IsValidIndex(Row))
						{
							switch (EnumColumn->RowValues[Row].Comparison)
							{
							case EEnumColumnCellValueComparison::MatchEqual:
								return LOCTEXT("CompEqual", "=");

							case EEnumColumnCellValueComparison::MatchNotEqual:
								return LOCTEXT("CompNotEqual", "Not");

							case EEnumColumnCellValueComparison::MatchAny:
								return LOCTEXT("CompAny", "Any");
							}
						}
						return FText::GetEmpty();
					})
					.OnClicked_Lambda([EnumColumn, Chooser, Row]()
					{
						if (EnumColumn->RowValues.IsValidIndex(Row))
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit Comparison", "Edit Comparison Operation"));
							Chooser->Modify(true);
							// cycle through comparison options
							EEnumColumnCellValueComparison& Comparison = EnumColumn->RowValues[Row].Comparison;
							const int32 NextComparison = (static_cast<int32>(Comparison) + 1) % static_cast<int32>(EEnumColumnCellValueComparison::Modulus);
							Comparison = static_cast<EEnumColumnCellValueComparison>(NextComparison);
						}
						return FReply::Handled();
					})
				]
			]
			+ SHorizontalBox::Slot().FillWidth(1)
			[
				SNew(SEnumCell)
					.Enum_Lambda([EnumColumn]() { return EnumColumn->GetEnum(); })
					.OnValueSet_Lambda([Chooser, EnumColumn, Row](int Value)
					{
						FScopedTransaction Transaction(LOCTEXT("Set Enum Value", "Set Enum Value"));
						Chooser->Modify();
						
						if (EnumColumn->RowValues.IsValidIndex(Row))
						{
							EnumColumn->RowValues[Row].Value = static_cast<uint8>(Value);

							if (EnumColumn->InputValue.IsValid())
							{
								if (const UEnum* Enum = EnumColumn->InputValue.Get<FChooserParameterEnumBase>().GetEnum())
								{
									EnumColumn->RowValues[Row].ValueName = Enum->GetNameByValue(Value);
								}
							}
						}
					})
					.EnumValue_Lambda([EnumColumn, Row]()
					{
						return EnumColumn->RowValues.IsValidIndex(Row) ? static_cast<int32>(EnumColumn->RowValues[Row].Value) : 0;
					})
					.Visibility_Lambda([EnumColumn,Column, Row]()
					{
						return (EnumColumn->RowValues.IsValidIndex(Row) &&
								EnumColumn->RowValues[Row].Comparison == EEnumColumnCellValueComparison::MatchAny)
								   ? EVisibility::Collapsed
								   : EVisibility::Visible;
					})
			];
}

TSharedRef<SWidget> CreateEnumPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FEnumContextProperty* ContextProperty = reinterpret_cast<FEnumContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(false).BindingColor("BytePinTypeColor").TypeFilter("enum")
	.PropertyBindingValue(&ContextProperty->Binding)
	.OnValueChanged(ValueChanged);
}

void CreateEnumPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FEnumContextProperty>("enum", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}
	
void RegisterEnumWidgets()
{
	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterEnumBase::StaticStruct(), CreateEnumPropertyMenus);
	
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FEnumContextProperty::StaticStruct(), CreateEnumPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FEnumColumn::StaticStruct(), CreateEnumColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
