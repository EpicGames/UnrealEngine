// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterEnum.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "EnumColumnEditor"

namespace UE::ChooserEditor
{
	// Enum combo box widget with a dynamic UEnum
	class SEnumCell : public SCompoundWidget
	{
	public:

		DECLARE_DELEGATE_OneParam(FOnValueSet, int);
		
		SLATE_BEGIN_ARGS(SEnumCell)
		{}

		SLATE_ATTRIBUTE(const UEnum*, Enum);
		SLATE_ATTRIBUTE(int32, EnumValue);
		SLATE_EVENT(FOnValueSet, OnValueSet)
				
		SLATE_END_ARGS()

		TSharedRef<SWidget> GenerateEnumMenu() const;
		TSharedRef<SWidget> CreateEnumComboBox();
		
		void Construct( const FArguments& InArgs);

		virtual ~SEnumCell() = default;

	private:
		FOnValueSet OnValueSet;
		TAttribute<int32> EnumValue;
		TAttribute<const UEnum*> Enum;
	};
	
	void RegisterEnumWidgets();
}

#undef LOCTEXT_NAMESPACE
