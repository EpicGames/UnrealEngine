// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatDistanceColumnEditor.h"

#include "ChooserEditorStyle.h"
#include "ChooserColumnHeader.h"
#include "FloatDistanceColumn.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserTableEditor.h"
#include "ChooserEditorStyle.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Images/SImage.h"
#include "GraphEditorSettings.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatDistanceColumnEditor)

#define LOCTEXT_NAMESPACE "FloatDistanceColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateFloatDistanceColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FFloatDistanceColumn* FloatDistanceColumn = static_cast<FFloatDistanceColumn*>(Column);

	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.SortUp");
		const FText ColumnTooltip = LOCTEXT("Float difference tooltip", "Float Difference Column: rows receive a Score based on how different the input float is from the row value");
		const FText ColumnName = LOCTEXT("Float Difference","Float Difference");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SNumericEntryBox<float>)
				.IsEnabled_Lambda([Chooser](){ return !Chooser->HasDebugTarget(); })
				.Value_Lambda([FloatDistanceColumn]() { return FloatDistanceColumn->TestValue; })
				.OnValueCommitted_Lambda([Chooser, FloatDistanceColumn](float NewValue, ETextCommit::Type CommitType) { FloatDistanceColumn->TestValue = NewValue; });
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SNumericEntryBox<float>)
	.Value_Lambda([FloatDistanceColumn, Row]()
	{
		return (FloatDistanceColumn->RowValues.IsValidIndex(Row)) ? FloatDistanceColumn->RowValues[Row].Value : 0;
	})
	.OnValueCommitted_Lambda([Chooser, FloatDistanceColumn, Row](float NewValue, ETextCommit::Type CommitType)
	{
		if (FloatDistanceColumn->RowValues.IsValidIndex(Row))
		{
			const FScopedTransaction Transaction(LOCTEXT("Edit Float Distance Value", "Edit Float Distance Value"));
			Chooser->Modify(true);
			FloatDistanceColumn->RowValues[Row].Value = NewValue;
		}
	});
}

	
void RegisterFloatDistanceWidgets()
{
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FFloatDistanceColumn::StaticStruct(), CreateFloatDistanceColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
