// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FloatDistanceColumn.h"
#include "FloatDistanceColumnEditor.generated.h"

namespace UE::ChooserEditor
{
	void RegisterFloatDistanceWidgets();
}

UCLASS(Blueprintable)
class UFloatAutoPopulatorBlueprint : public UFloatAutoPopulator
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintImplementableEvent)
	void AutoPopulate(UObject* Object, bool& Success, float& Value);
	
	virtual void NativeAutoPopulate(UObject* InObject, bool& OutSuccess, float& OutValue) { AutoPopulate(InObject, OutSuccess, OutValue); }
};
