// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatRangeColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "FloatRangeColumn.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserTableEditor.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Images/SImage.h"
#include "GraphEditorSettings.h"
#include "SPropertyAccessChainWidget.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Features/IModularFeatures.h"

#include "IPropertyAccessEditor.h"

#define LOCTEXT_NAMESPACE "FloatRangeColumnEditor"

namespace UE::ChooserEditor
{


TSharedRef<SWidget> CreateFloatPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FFloatContextProperty* ContextProperty = reinterpret_cast<FFloatContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(true).BindingColor("FloatPinTypeColor").TypeFilter("double")
	.PropertyBindingValue(&ContextProperty->Binding)
	.OnValueChanged(ValueChanged);
}
	
TSharedRef<SWidget> CreateFloatRangeColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FFloatRangeColumn* FloatRangeColumn = static_cast<FFloatRangeColumn*>(Column);

	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Float Range Tooltip", "Float Range: cells pass if the input value is within the range specified in the cell");
		const FText ColumnName = LOCTEXT("Float Range","Float Range");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
            DebugWidget = SNew(SNumericEntryBox<float>).IsEnabled_Lambda([Chooser](){ return !Chooser->HasDebugTarget(); })
            				.Value_Lambda([FloatRangeColumn]() { return FloatRangeColumn->TestValue; })
            				.OnValueCommitted_Lambda([Chooser, FloatRangeColumn](float NewValue, ETextCommit::Type CommitType)
            				{
            					FloatRangeColumn->TestValue = NewValue;
            				});
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
	[
		SNew(STextBlock).Text(LOCTEXT("FloatRangeLeft", "("))
	]
	+ SHorizontalBox::Slot().FillWidth(0.5f)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.Visibility_Lambda([FloatRangeColumn, Row]
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row) && FloatRangeColumn->RowValues[Row].bNoMin) ? EVisibility::Hidden : EVisibility::Visible;
			})
			.MaxValue_Lambda([FloatRangeColumn, Row]()
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row)) ? FloatRangeColumn->RowValues[Row].Max : 0;
			})
			.Value_Lambda([FloatRangeColumn, Row]()
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row)) ? FloatRangeColumn->RowValues[Row].Min : 0;
			})
			.OnValueCommitted_Lambda([Chooser, FloatRangeColumn, Row](float NewValue, ETextCommit::Type CommitType)
			{
				if (FloatRangeColumn->RowValues.IsValidIndex(Row))
				{
					const FScopedTransaction Transaction(LOCTEXT("Edit Min Value", "Edit Min Value"));
					Chooser->Modify(true);
					FloatRangeColumn->RowValues[Row].Min = NewValue;
				}
			})
		]
		+ SOverlay::Slot()
		[
			SNew(SEditableTextBox).IsReadOnly(true).Text(LOCTEXT("negativeinfinity","-∞"))
				.Visibility_Lambda([FloatRangeColumn, Row]
				{
					return (FloatRangeColumn->RowValues.IsValidIndex(Row) && FloatRangeColumn->RowValues[Row].bNoMin) ? EVisibility::Visible : EVisibility::Hidden;
				})
		]
	]
	+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
	[
		SNew(STextBlock).Text(LOCTEXT("FloatRangeComma", " ,"))
	]
	+ SHorizontalBox::Slot().FillWidth(0.5f)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.Visibility_Lambda([FloatRangeColumn, Row]
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row) && FloatRangeColumn->RowValues[Row].bNoMax) ? EVisibility::Hidden : EVisibility::Visible;
			})
			.MinValue_Lambda([FloatRangeColumn, Row]()
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row)) ? FloatRangeColumn->RowValues[Row].Min : 0;
			})
			.Value_Lambda([FloatRangeColumn, Row]()
			{
				return (FloatRangeColumn->RowValues.IsValidIndex(Row)) ? FloatRangeColumn->RowValues[Row].Max : 0;
			})
			.OnValueCommitted_Lambda([Chooser, FloatRangeColumn, Row](float NewValue, ETextCommit::Type CommitType)
			{
				if (FloatRangeColumn->RowValues.IsValidIndex(Row))
				{
					const FScopedTransaction Transaction(LOCTEXT("Edit Max", "Edit Max Value"));
					Chooser->Modify(true);
					FloatRangeColumn->RowValues[Row].Max = NewValue;
				}
			})
		]
		+ SOverlay::Slot()
		[
			SNew(SEditableTextBox).IsReadOnly(true).Text(LOCTEXT("infinity","∞"))
				.Visibility_Lambda([FloatRangeColumn, Row]
				{
					return  (FloatRangeColumn->RowValues.IsValidIndex(Row) && FloatRangeColumn->RowValues[Row].bNoMax) ? EVisibility::Visible : EVisibility::Hidden;
				})
		]
	]
	+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
	[
		SNew(STextBlock).Text(LOCTEXT("FloatRangeRight", " )"))
	];
}
		
void CreateFloatPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FFloatContextProperty>("double", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}
	
void RegisterFloatRangeWidgets()
{
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FFloatContextProperty::StaticStruct(), CreateFloatPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FFloatRangeColumn::StaticStruct(), CreateFloatRangeColumnWidget);

	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterFloatBase::StaticStruct(), CreateFloatPropertyMenus);
}
	
}

#undef LOCTEXT_NAMESPACE
