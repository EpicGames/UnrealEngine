// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameTimeCustomization.h"

#include "AnimNode_ChooserPlayer.h"
#include "AudioDevice.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDocumentation.h"
#include "ScopedTransaction.h"
#include "SEnumCombo.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FrameTimeCustomization)

#define LOCTEXT_NAMESPACE "FrameTimeCustomization"

namespace UE::ChooserEditor
{
	
static const float SecondsToFrames = 30.0f;
static const float FramesToSeconds = 1.0f/30.0f;
static ETimeFloatFormat GTimeFloatDisplayFormat = ETimeFloatFormat::Frames;
	
void FFrameTimeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	UEnum* TimeFloatFormatEnum = StaticEnum<ETimeFloatFormat>();
	HeaderRow
    	.NameContent()
    	[
    		PropertyHandle->CreatePropertyNameWidget()
    	]
    	.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SBox).WidthOverride(125).HeightOverride(20)
             	[
					SNew(SNumericEntryBox<float>)
						.MinFractionalDigits(0)
						.MaxFractionalDigits(3)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.BoldFont"))
						.OverrideTextMargin(FMargin(8,0,8,0))
						.Value_Lambda([PropertyHandle]()
						{
							float Value;
							PropertyHandle->GetValue(Value);
							return GTimeFloatDisplayFormat == ETimeFloatFormat::Frames ? Value * SecondsToFrames : Value;
						})
						.OnValueCommitted_Lambda([PropertyHandle](float NewValue, ETextCommit::Type CommitType)
						{
							FScopedTransaction ScopedTransaction(LOCTEXT("Set Frame Time Value", "Set Frame Time Value"));
							PropertyHandle->NotifyPreChange();
							PropertyHandle->EnumerateRawData([NewValue](void* RawData,int32 Index, int32 Num)
							{
								*static_cast<float*>(RawData) = GTimeFloatDisplayFormat == ETimeFloatFormat::Frames ? NewValue*FramesToSeconds : NewValue;
								return true;
							});
							PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
							
						})
				]
			]
			+SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SBox).WidthOverride(80).HeightOverride(20)
				[
					SNew(SEnumComboBox, TimeFloatFormatEnum)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.CurrentValue_Lambda([]{return static_cast<int32>(GTimeFloatDisplayFormat);})
					.ContentPadding(FMargin(0.0f, 0.0f))
					.ToolTipText(LOCTEXT("FrameTimeToolTip","Select whether to display and edit the time as Seconds, or Frames (at 30 fps)"))
					.OnEnumSelectionChanged_Lambda([](uint32 NewValue, ESelectInfo::Type)
					{
						FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
						GTimeFloatDisplayFormat = static_cast<ETimeFloatFormat>(NewValue);
					})
				]
			]
		];
}


void FFrameTimeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE
