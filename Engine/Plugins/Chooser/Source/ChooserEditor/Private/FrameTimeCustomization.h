// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "FrameTimeCustomization.generated.h"

UENUM()
enum class ETimeFloatFormat
{
	Frames,
	Seconds
};

namespace UE::ChooserEditor
{

class FFrameTimePropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return PropertyHandle.HasMetaData(TEXT("FrameTimeEditor"));
	}
};

class FFrameTimeCustomization : public IPropertyTypeCustomization
{
public:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};

}