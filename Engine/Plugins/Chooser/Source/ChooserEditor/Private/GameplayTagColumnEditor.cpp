// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "SPropertyAccessChainWidget.h"
#include "GameplayTagColumn.h"
#include "ObjectChooserWidgetFactories.h"
#include "SGameplayTagWidget.h"
#include "SSimpleComboButton.h"
#include "GraphEditorSettings.h"
#include "OutputGameplayTagColumn.h"
#include "SGameplayTagContainerCombo.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "FGameplayTagColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateGameplayTagColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FGameplayTagColumn* GameplayTagColumn = static_cast<struct FGameplayTagColumn*>(Column);

	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Gameplay Tag Tooltip", "Gameplay Tag: cells pass if the input gameplay tag collection matches the cell data (according to comparison settings in the column properties).");
		const FText ColumnName = LOCTEXT("Gameplay Tag","Gameplay Tag");

		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SGameplayTagContainerCombo)
				.IsEnabled_Lambda([Chooser]()
				{
					 return !Chooser->HasDebugTarget();
				})
				.TagContainer_Lambda([GameplayTagColumn]()
				{
					return GameplayTagColumn->TestValue;
				})
				.OnTagContainerChanged_Lambda([GameplayTagColumn](const FGameplayTagContainer& UpdatedTags)
				{
					GameplayTagColumn->TestValue = UpdatedTags;
				});
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SGameplayTagContainerCombo)
		.ReadOnly(false)
		.TagContainer_Lambda([GameplayTagColumn, Row]()
		{
			if (GameplayTagColumn->RowValues.IsValidIndex(Row))
			{
				return GameplayTagColumn->RowValues[Row];
			}
			else
			{
				return FGameplayTagContainer::EmptyContainer;
			}
		})
		.OnTagContainerChanged_Lambda([GameplayTagColumn, Row](const FGameplayTagContainer& UpdatedTags)
		{
			if (GameplayTagColumn->RowValues.IsValidIndex(Row))
			{
				GameplayTagColumn->RowValues[Row] = UpdatedTags;
			}
		}
	);
}

TSharedRef<SWidget> CreateOutputGameplayTagColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FOutputGameplayTagColumn* OutputGameplayTagColumn = static_cast<struct FOutputGameplayTagColumn*>(Column);

	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Gameplay Tag Tooltip", "Output Gameplay Tag: writes the value from cell in the result row to the bound variable");
		const FText ColumnName = LOCTEXT("Output Gameplay Tag", "Output Gameplay Tag");

		TSharedPtr<SWidget> DebugWidget = nullptr;
		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SGameplayTagContainerCombo)
		.ReadOnly(false)
		.TagContainer_Lambda([OutputGameplayTagColumn, Row]()
		{
			if (OutputGameplayTagColumn->RowValues.IsValidIndex(Row))
			{
				return OutputGameplayTagColumn->RowValues[Row];
			}
			else
			{
				return FGameplayTagContainer::EmptyContainer;
			}
		})
		.OnTagContainerChanged_Lambda([OutputGameplayTagColumn, Row](const FGameplayTagContainer& UpdatedTags)
		{
			if (OutputGameplayTagColumn->RowValues.IsValidIndex(Row))
			{
				OutputGameplayTagColumn->RowValues[Row] = UpdatedTags;
			}
		}
	);
}

TSharedRef<SWidget> CreateGameplayTagPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FGameplayTagContextProperty* ContextProperty = reinterpret_cast<FGameplayTagContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(false).BindingColor("StructPinTypeColor").TypeFilter("FGameplayTagContainer")
	.PropertyBindingValue(&ContextProperty->Binding)
	.OnValueChanged(ValueChanged);
}
	
void CreateGameplayTagPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FGameplayTagContextProperty>("FGameplayTagContainer", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}

void RegisterGameplayTagWidgets()
{
	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterGameplayTagBase::StaticStruct(), CreateGameplayTagPropertyMenus);
	
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FGameplayTagContextProperty::StaticStruct(), CreateGameplayTagPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FGameplayTagColumn::StaticStruct(), CreateGameplayTagColumnWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputGameplayTagColumn::StaticStruct(), CreateOutputGameplayTagColumnWidget);
}

}

#undef LOCTEXT_NAMESPACE
