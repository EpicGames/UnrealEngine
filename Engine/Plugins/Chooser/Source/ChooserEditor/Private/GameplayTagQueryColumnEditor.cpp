// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagQueryColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "GameplayTagQueryColumn.h"
#include "ObjectChooserWidgetFactories.h"
#include "OutputGameplayTagQueryColumn.h"
#include "SGameplayTagQueryEntryBox.h"
#include "SGameplayTagWidget.h"
#include "SPropertyAccessChainWidget.h"
#include "SSimpleComboButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "FGameplayTagQueryColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateGameplayTagQueryColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FGameplayTagQueryColumn* GameplayTagQueryColumn = static_cast<struct FGameplayTagQueryColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Gameplay Tag Query Tooltip", "Gameplay Tag Query: cells pass if the input gameplay tag collection matches the query specified in the column properties. Note that empty queries never pass.");
		const FText ColumnName = LOCTEXT("Gameplay Tag Query","Gameplay Tag Query");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SSimpleComboButton)
				.IsEnabled_Lambda([Chooser]()
				{
					 return !Chooser->HasDebugTarget();
				})
				.Text_Lambda([GameplayTagQueryColumn]()
				{
					FText Text = FText::FromString(GameplayTagQueryColumn->TestValue.ToStringSimple(false));
					if (Text.IsEmpty())
					{
						Text = LOCTEXT("None", "None");
					}
					return Text;
				})	
				.OnGetMenuContent_Lambda([Chooser, GameplayTagQueryColumn]()
				{
					TArray<SGameplayTagWidget::FEditableGameplayTagContainerDatum> EditableContainers;
					EditableContainers.Emplace(Chooser, &(GameplayTagQueryColumn->TestValue));
					return TSharedRef<SWidget>(SNew(SGameplayTagWidget, EditableContainers));
				});
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SGameplayTagQueryEntryBox)
		.TagQuery_Lambda([GameplayTagQueryColumn, Row]()
		{
			if (GameplayTagQueryColumn->RowValues.IsValidIndex(Row))
			{
				return GameplayTagQueryColumn->RowValues[Row];
			}
			return FGameplayTagQuery::EmptyQuery;
		})
		.ReadOnly(false)
		.DescriptionMaxWidth(1000.f)
		.OnTagQueryChanged_Lambda([GameplayTagQueryColumn, Row](const FGameplayTagQuery& UpdatedQuery)
		{
			if (GameplayTagQueryColumn->RowValues.IsValidIndex(Row))
			{
				GameplayTagQueryColumn->RowValues[Row] = UpdatedQuery;
			}
		});
}

TSharedRef<SWidget> CreateOutputGameplayTagQueryColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FOutputGameplayTagQueryColumn* OutputGameplayTagQueryColumn = static_cast<struct FOutputGameplayTagQueryColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	else if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Gameplay Tag Query Tooltip", "Output Gameplay Tag Query: writes the value from cell in the result row to the bound variable");
		const FText ColumnName = LOCTEXT("Output Gameplay Tag Query", "Output Gameplay Tag Query");
		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return SNew(SGameplayTagQueryEntryBox)
		.TagQuery_Lambda([OutputGameplayTagQueryColumn, Row]()
		{
			if (OutputGameplayTagQueryColumn->RowValues.IsValidIndex(Row))
			{
				return OutputGameplayTagQueryColumn->RowValues[Row];
			}
			return FGameplayTagQuery::EmptyQuery;
		})
		.ReadOnly(false)
		.DescriptionMaxWidth(1000.f)
		.OnTagQueryChanged_Lambda([OutputGameplayTagQueryColumn, Row](const FGameplayTagQuery& UpdatedQuery)
		{
			if (OutputGameplayTagQueryColumn->RowValues.IsValidIndex(Row))
			{
				OutputGameplayTagQueryColumn->RowValues[Row] = UpdatedQuery;
			}
		});
}

TSharedRef<SWidget> CreateGameplayTagQueryPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FGameplayTagQueryContextProperty* ContextProperty = reinterpret_cast<FGameplayTagQueryContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).BindingColor("StructPinTypeColor").TypeFilter("FGameplayTagQuery")
		.PropertyBindingValue(&ContextProperty->Binding)
		.OnValueChanged(ValueChanged);
}
	
void CreateGameplayTagQueryPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FGameplayTagQueryContextProperty>("FGameplayTagQuery", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}
	
void RegisterGameplayTagQueryWidgets()
{
	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterGameplayTagQueryBase::StaticStruct(), CreateGameplayTagQueryPropertyMenus);
	
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FGameplayTagQueryContextProperty::StaticStruct(), CreateGameplayTagQueryPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FGameplayTagQueryColumn::StaticStruct(), CreateGameplayTagQueryColumnWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputGameplayTagQueryColumn::StaticStruct(), CreateOutputGameplayTagQueryColumnWidget);
	// No need to make and register a creator for gameplay tag containers - it's already registered in GameplayTagColumnEditor
}
	
}

#undef LOCTEXT_NAMESPACE
