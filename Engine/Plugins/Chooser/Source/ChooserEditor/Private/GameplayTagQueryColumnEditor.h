// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::ChooserEditor
{
	void RegisterGameplayTagQueryWidgets();
}
