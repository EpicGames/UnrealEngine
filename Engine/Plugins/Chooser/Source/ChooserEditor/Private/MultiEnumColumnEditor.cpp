// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiEnumColumnEditor.h"
#include "EnumColumnEditor.h"
#include "MultiEnumColumn.h"
#include "SPropertyAccessChainWidget.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserColumnHeader.h"
#include "ChooserTableEditor.h"
#include "GraphEditorSettings.h"
#include "SEnumCombo.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Input/SButton.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MultiEnumColumnEditor"

namespace UE::ChooserEditor
{

	
// Wrapper widget for EnumComboBox which will reconstruct the combo box when the Enum has changed
template <typename ColumnType>
class SMultiEnumCell : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnValueSet, int);
	
	SLATE_BEGIN_ARGS(SMultiEnumCell)
	{}

	SLATE_ARGUMENT(UObject*, TransactionObject)
	SLATE_ARGUMENT(ColumnType*, MultiEnumColumn)
	SLATE_ATTRIBUTE(int32, EnumValue);
	SLATE_EVENT(FOnValueSet, OnValueSet)
            
	SLATE_END_ARGS()

	TSharedRef<SWidget> CreateEnumComboBox()
	{
		if (const ColumnType* MultiEnumColumnPointer = MultiEnumColumn)
		{
			if (MultiEnumColumnPointer->InputValue.IsValid())
			{
				if (const UEnum* Enum = MultiEnumColumnPointer->InputValue.template Get<FChooserParameterEnumBase>().GetEnum())
				{
					return SNew(SEnumComboBox, Enum)
						.bForceBitFlags(true)
						.OverrideNoFlagsSetText(LOCTEXT("(Any)", "(Any)"))
						.IsEnabled_Lambda([this](){ return IsEnabled(); } )
						.CurrentValue(EnumValue)
						.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type)
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit RHS", "Edit Enum Value"));
							TransactionObject->Modify(true);
							OnValueSet.ExecuteIfBound(InEnumValue);
						});
				}
			}
		}
		
		return SNullWidget::NullWidget;
	}

	void UpdateEnumComboBox()
	{
		ChildSlot[ CreateEnumComboBox()	];
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		const UEnum* CurrentEnumSource = nullptr;
		if (MultiEnumColumn->InputValue.IsValid())
		{
			CurrentEnumSource = MultiEnumColumn->InputValue.template Get<FChooserParameterEnumBase>().GetEnum(); 
		}
		if (EnumSource != CurrentEnumSource)
		{
			EnumComboBorder->SetContent(CreateEnumComboBox());
			EnumSource = CurrentEnumSource;
		}
	}
    					

	void Construct( const FArguments& InArgs)
	{
		SetEnabled(InArgs._IsEnabled);
		
		SetCanTick(true);
		MultiEnumColumn = InArgs._MultiEnumColumn;
		TransactionObject = InArgs._TransactionObject;
		EnumValue = InArgs._EnumValue;
		OnValueSet = InArgs._OnValueSet;

		if (MultiEnumColumn)
		{
			if (MultiEnumColumn->InputValue.IsValid())
			{
				EnumSource = MultiEnumColumn->InputValue.template Get<FChooserParameterEnumBase>().GetEnum();
			}
		}

		UpdateEnumComboBox();

		int Row = RowIndex.Get();

		ChildSlot
		[
			SAssignNew(EnumComboBorder, SBorder).Padding(0).BorderBackgroundColor(FLinearColor(0,0,0,0))
			[
				CreateEnumComboBox()
			]
		];
		
	}

	~SMultiEnumCell()
	{
	}

private:
	UObject* TransactionObject = nullptr;
	ColumnType* MultiEnumColumn = nullptr;
	const UEnum* EnumSource = nullptr;
	TSharedPtr<SBorder> EnumComboBorder;
	TAttribute<int> RowIndex;
	FDelegateHandle EnumChangedHandle;
	
	FOnValueSet OnValueSet;
	TAttribute<int32> EnumValue;
};

TSharedRef<SWidget> CreateMultiEnumColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FMultiEnumColumn* MultiEnumColumn = static_cast<FMultiEnumColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
		const FText ColumnTooltip = LOCTEXT("Multi Enum Tooltip", "Enum Any: cells will pass if the input value is any of the enum values checked in the cell");
		const FText ColumnName = LOCTEXT("Enum Or","Enum (Or)");
		
	
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SEnumCell)
							.Enum_Lambda([MultiEnumColumn] () { return MultiEnumColumn->GetEnum(); })
							.OnValueSet_Lambda([MultiEnumColumn](int Value) { MultiEnumColumn->TestValue = Value; })
							.EnumValue_Lambda([MultiEnumColumn]() { return MultiEnumColumn->TestValue; })
							.IsEnabled_Lambda([Chooser] { return !Chooser->HasDebugTarget(); });
			
			// need to fix support for bitfield enums:
			// DebugWidget = SNew(SMultiEnumCell<FMultiEnumColumn>).TransactionObject(Chooser).MultiEnumColumn(MultiEnumColumn)
   //                  					.OnValueSet_Lambda([MultiEnumColumn](int Value) { MultiEnumColumn->TestValue = Value; })
   //                  					.EnumValue_Lambda([MultiEnumColumn]() { return MultiEnumColumn->TestValue; })
   //                  					.IsEnabled_Lambda([Chooser] { return !Chooser->HasDebugTarget(); });
		}

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	
	return SNew(SMultiEnumCell<FMultiEnumColumn>).TransactionObject(Chooser).MultiEnumColumn(MultiEnumColumn)
			.OnValueSet_Lambda([MultiEnumColumn, Row](int Value)
			{
				if (MultiEnumColumn->RowValues.IsValidIndex(Row))
				{
					MultiEnumColumn->RowValues[Row].Value = static_cast<uint32>(Value);
				}
			})
			.EnumValue_Lambda([MultiEnumColumn, Row]()
			{
				return MultiEnumColumn->RowValues.IsValidIndex(Row) ? static_cast<int32>(MultiEnumColumn->RowValues[Row].Value) : 0;
			});
}

void RegisterMultiEnumWidgets()
{
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FMultiEnumColumn::StaticStruct(), CreateMultiEnumColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
