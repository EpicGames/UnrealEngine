// Copyright Epic Games, Inc. All Rights Reserved.

#include "NameColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "SPropertyAccessChainWidget.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "NameColumn.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NameColumnEditor"

namespace UE::ChooserEditor
{
	static TSharedRef<SWidget> CreateNameColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int32 Row)
	{
		FChooserNameColumn* NameColumn = static_cast<FChooserNameColumn*>(Column);

		if (Row == ColumnWidget_SpecialIndex_Fallback)
		{
			return SNullWidget::NullWidget;
		}
		else if (Row == ColumnWidget_SpecialIndex_Header)
		{
			const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
			const FText ColumnTooltip = LOCTEXT("Name Tooltip", "Name: cells pass if the Name in the cell matches the input Name, using the compare type");
			const FText ColumnName = LOCTEXT("Name","Name");
			
			TSharedPtr<SWidget> DebugWidget = nullptr;
			if (Chooser->GetEnableDebugTesting())
			{
				DebugWidget = SNew(SEditableTextBox).IsEnabled_Lambda([Chooser](){ return !Chooser->HasDebugTarget(); })
								.Text_Lambda([NameColumn]() { return FText::FromName(NameColumn->TestValue); })
								.OnTextCommitted_Lambda([Chooser, NameColumn](const FText& NewValue, ETextCommit::Type CommitType)
								{
									NameColumn->TestValue = FName(NewValue.ToString());
								});
			}
			return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
		}


		// create widget for cell
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(55)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "FlatButton")
					.TextStyle(FAppStyle::Get(), "RichTextBlock.Bold")
					.HAlign(HAlign_Center)
					.Text_Lambda([NameColumn, Row]() {
						if (NameColumn->RowValues.IsValidIndex(Row))
						{
							switch (NameColumn->RowValues[Row].Comparison)
							{
								case ENameColumnCellValueComparison::MatchEqual:
									return LOCTEXT("CompEqual", "=");

								case ENameColumnCellValueComparison::MatchNotEqual:
									return LOCTEXT("CompNotEqual", "Not");

								case ENameColumnCellValueComparison::MatchAny:
									return LOCTEXT("CompAny", "Any");
							}
						}
						return FText::GetEmpty();
					})
					.OnClicked_Lambda([Chooser, NameColumn, Row]() {
						if (NameColumn->RowValues.IsValidIndex(Row))
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit Comparison", "Edit Comparison Operation"));
							Chooser->Modify(true);
							// cycle through comparison options
							ENameColumnCellValueComparison& Comparison = NameColumn->RowValues[Row].Comparison;
							const int32 NextComparison = (static_cast<int32>(Comparison) + 1) % static_cast<int32>(ENameColumnCellValueComparison::Modulus);
							Comparison = static_cast<ENameColumnCellValueComparison>(NextComparison);
						}
						return FReply::Handled();
					})
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
					.Visibility_Lambda([NameColumn, Row]()
					{
						return (NameColumn->RowValues.IsValidIndex(Row) && NameColumn->RowValues[Row].Comparison == ENameColumnCellValueComparison::MatchAny)
							   ? EVisibility::Collapsed
							   : EVisibility::Visible;	
					})
					.Text_Lambda([NameColumn, Row]()
					{
						return NameColumn->RowValues.IsValidIndex(Row) ? FText::FromName(NameColumn->RowValues[Row].Value) : FText();
					})
					.OnTextCommitted_Lambda([Chooser, NameColumn, Row](const FText& NewValue, ETextCommit::Type CommitType)
					{
						const FScopedTransaction Transaction(LOCTEXT("Edit Name", "Edit Name"));
						Chooser->Modify(true);
						if (NameColumn->RowValues.IsValidIndex(Row))
						{
							NameColumn->RowValues[Row].Value = FName(NewValue.ToString());
						}
					})
			];
	}

	void CreateNamePropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
    {
    	CreatePropertyAccessMenus<FNameContextProperty>("FName", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
     }

	void RegisterNameWidgets()
	{
		FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterNameBase::StaticStruct(), CreateNamePropertyMenus);
		FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FChooserNameColumn::StaticStruct(), CreateNameColumnWidget);
	}

} // namespace UE::ChooserEditor

#undef LOCTEXT_NAMESPACE
