// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectChooserWidgetFactories.h"

#include "DetailCategoryBuilder.h"
#include "IObjectChooser.h"
#include "ObjectChooserClassFilter.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "StructViewerModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "DataInterfaceEditor"

namespace UE::ChooserEditor
{
	
void FObjectChooserWidgetFactories::RegisterWidgetCreator(const UStruct* Type, FNewChooserWidgetCreator Creator)
{
	NewChooserWidgetCreators.Add(Type, Creator);	
}
	
void FObjectChooserWidgetFactories::RegisterWidgetCreator(const UStruct* Type, FChooserWidgetCreator Creator)
{
	ChooserWidgetCreators.Add(Type, Creator);	
}
	
void  FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(const UStruct* ColumnType, FColumnWidgetCreator Creator)
{
	ColumnWidgetCreators.Add(ColumnType, Creator);	
}

void FObjectChooserWidgetFactories::RegisterParameterMenuCreator(const UStruct* ParameterBaseType, FParameterMenuCreator Creator)
{
	ParameterMenuCreators.Add(ParameterBaseType, Creator);
}

void FObjectChooserWidgetFactories::CreateParametersMenu(const UStruct* ParameterBaseType, UObject* TransactionObject, const IHasContextClass* ContextOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	for (FParameterMenuCreatorMap::TConstKeyIterator It = ParameterMenuCreators.CreateConstKeyIterator(ParameterBaseType); It; ++It)
	{
		It->Value(TransactionObject, ContextOwner, Parameter, MenuBuilder, BindingChanged);
	}	
}

TSharedPtr<SWidget> FObjectChooserWidgetFactories::CreateColumnWidget(FChooserColumnBase* Column, const UStruct* ColumnType, UChooserTable* Chooser, int RowIndex)
{
	if (Column)
	{
		while (ColumnType)
		{
			if (FColumnWidgetCreator* Creator = FObjectChooserWidgetFactories::ColumnWidgetCreators.Find(ColumnType))
			{
				return (*Creator)(Chooser, Column, RowIndex);
			}
			ColumnType = ColumnType->GetSuperStruct();
		}
	}

	return nullptr;
}


TSharedPtr<SWidget> FObjectChooserWidgetFactories::CreateWidget(bool bReadOnly, UObject* TransactionObject, void* Value, const UStruct* ValueType, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged, IChooserTableWidgetInterface* ChooserWidgetInterface)
{
	if (Value)
	{
		while (ValueType)
		{
			
			if (FNewChooserWidgetCreator* NewCreator = NewChooserWidgetCreators.Find(ValueType))
			{
				return (*NewCreator)(bReadOnly, TransactionObject, Value, ResultBaseClass, ValueChanged, ChooserWidgetInterface);
			}
			else if (FChooserWidgetCreator* Creator = ChooserWidgetCreators.Find(ValueType))
			{
				return (*Creator)(bReadOnly, TransactionObject, Value, ResultBaseClass, ValueChanged);
			}
			ValueType = ValueType->GetSuperStruct();
		}
	}

	return nullptr;
}

class SObjectChooserWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SObjectChooserWidget)
	{
		_TransactionObject = nullptr;
		_Data = nullptr;
		_NullValueDisplayText =  LOCTEXT("SelectDataType", "Select Data Type...");
		_bReadOnly = false;
		
	}

	SLATE_ARGUMENT(UObject*, TransactionObject)
	SLATE_ARGUMENT(FInstancedStruct*, Data)
	SLATE_ARGUMENT(UScriptStruct*, DataBaseType)
	SLATE_ARGUMENT(UClass*, ResultBaseClass)
	SLATE_ARGUMENT(FText, NullValueDisplayText)
	SLATE_ARGUMENT(bool, bReadOnly)
	SLATE_EVENT(FChooserWidgetValueChanged, ValueChanged);
	SLATE_ARGUMENT(IChooserTableWidgetInterface*, ChooserWidgetInterface);
			
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs)
	{
		TransactionObject = InArgs._TransactionObject;
		Data = InArgs._Data;
		DataBaseType = InArgs._DataBaseType;
		ResultBaseClass = InArgs._ResultBaseClass;
		bReadOnly = InArgs._bReadOnly;
		ValueChanged = InArgs._ValueChanged;
		ChooserWidgetInterface = InArgs._ChooserWidgetInterface;

		if (!InArgs._NullValueDisplayText.IsEmpty())
		{
			NullValueDisplayText = InArgs._NullValueDisplayText;
		}

		Border = SNew(SBorder);
		UpdateValueWidget();
		
		TSharedPtr<SWidget> Widget = Border;
		
		// don't need the type selector dropdown when read only
		if (!bReadOnly)
		{
			// button for replacing data with a different Data Interface class
			TSharedPtr<SComboButton> Button = SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton");
		
			Button->SetOnGetMenuContent(FOnGetContent::CreateLambda([this, Button]()
			{
				FStructViewerInitializationOptions Options;
				Options.StructFilter = MakeShared<FStructFilter>(DataBaseType);
				Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
				Options.bShowNoneOption = true;
			
				TSharedRef<SWidget> StructViewerWidget = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, FOnStructPicked::CreateLambda(
					[this, Button](const UScriptStruct* ChosenStruct)
				{
					Button->SetIsOpen(false);
					const FScopedTransaction Transaction(LOCTEXT("Change Object Type", "Change Object Type"));
					TransactionObject->Modify(true);
					Data->InitializeAs(ChosenStruct);
					UpdateValueWidget();
				}));
				return StructViewerWidget;
			}));

			Widget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot().FillWidth(100)
			[
				Border.ToSharedRef()
			]
			+SHorizontalBox::Slot().AutoWidth()
			[
				Button.ToSharedRef()
			];
		}

		ChildSlot
		[
			Widget.ToSharedRef()
		];
		
	}

	~SObjectChooserWidget()
	{
	}

private:

	void UpdateValueWidget()
	{
		TSharedPtr<SWidget> NewWidget = FObjectChooserWidgetFactories::CreateWidget(bReadOnly, TransactionObject, Data->GetMutableMemory(), Data->GetScriptStruct(), ResultBaseClass, ValueChanged, ChooserWidgetInterface);

		if (!NewWidget.IsValid())
		{
			NewWidget = SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Margin(2)
				.Text(NullValueDisplayText);
		}
		
		Border->SetContent(NewWidget.ToSharedRef());
	}

	
	UObject* TransactionObject = nullptr;
	FInstancedStruct* Data = nullptr;
	UScriptStruct* DataBaseType = nullptr;
	UClass* ResultBaseClass = nullptr;
	TSharedPtr<SBorder> Border;
	bool bReadOnly = false;
	FChooserWidgetValueChanged ValueChanged;
	IChooserTableWidgetInterface* ChooserWidgetInterface = nullptr;
	FText NullValueDisplayText;
};

TSharedPtr<SWidget> FObjectChooserWidgetFactories::CreateWidget(bool bReadOnly, UObject* TransactionObject, UScriptStruct* DataBaseType, FInstancedStruct* Data, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged, IChooserTableWidgetInterface* ChooserWidgetInterface, FText NullValueDisplayText)
{
	return SNew(SObjectChooserWidget)
		.bReadOnly(bReadOnly)
		.TransactionObject(TransactionObject)
		.Data(Data).DataBaseType(DataBaseType)
		.ResultBaseClass(ResultBaseClass)
		.ValueChanged(ValueChanged)
		.ChooserWidgetInterface(ChooserWidgetInterface)
		.NullValueDisplayText(NullValueDisplayText);
}

TSharedPtr<SWidget> FObjectChooserWidgetFactories::CreateWidget(bool bReadOnly, UObject* TransactionObject, const UScriptStruct* BaseType, void* Value, const UStruct* ValueType, UClass* ResultBaseClass,
	const FOnStructPicked& CreateClassCallback, TSharedPtr<SBorder>* InnerWidget, FChooserWidgetValueChanged ValueChanged, IChooserTableWidgetInterface *ChooserWidgetInterface, FText NullValueDisplayText)
{
	TSharedPtr<SWidget> LeftWidget = CreateWidget(bReadOnly, TransactionObject, Value, ValueType, ResultBaseClass, ValueChanged, ChooserWidgetInterface);
	if (bReadOnly)
	{
		// don't need the type selector dropdown when read only
		return LeftWidget;
	}
	
	if (!LeftWidget.IsValid())
	{
		LeftWidget = SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Margin(2)
			.Text(NullValueDisplayText.IsEmpty() ? LOCTEXT("SelectDataType", "Select Data Type..." ) : NullValueDisplayText);
	}

	// button for replacing data with a different Data Interface class
	TSharedPtr<SComboButton> Button = SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton");
	
	Button->SetOnGetMenuContent(FOnGetContent::CreateLambda([BaseType, Button, CreateClassCallback]()
	{
		FStructViewerInitializationOptions Options;
		Options.StructFilter = MakeShared<FStructFilter>(BaseType);
		Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
		Options.bShowNoneOption = true;
		
		TSharedRef<SWidget> Widget = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, FOnStructPicked::CreateLambda(
			[Button, CreateClassCallback](const UScriptStruct* ChosenStruct)
		{
			Button->SetIsOpen(false);
			CreateClassCallback.Execute(ChosenStruct);
		}));
		return Widget;
	}));

	TSharedPtr <SBorder> Border;
	if (InnerWidget && InnerWidget->IsValid())
	{
		Border = *InnerWidget;
	}
	else
	{
		Border = SNew(SBorder);
	}
	
	if (InnerWidget)
	{
		*InnerWidget = Border;
	}
	
	Border->SetContent(LeftWidget.ToSharedRef());

	TSharedPtr<SWidget> Widget = SNew(SHorizontalBox)
		+SHorizontalBox::Slot().FillWidth(100)
		[
			Border.ToSharedRef()
		]
		+SHorizontalBox::Slot().AutoWidth()
		[
			Button.ToSharedRef()
		]
	;

	return Widget;
}

TMap<const UStruct*, TFunction<TSharedRef<SWidget>(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)>> FObjectChooserWidgetFactories::ColumnWidgetCreators;
TMap<const UStruct*, FNewChooserWidgetCreator> FObjectChooserWidgetFactories::NewChooserWidgetCreators;
TMap<const UStruct*, FChooserWidgetCreator> FObjectChooserWidgetFactories::ChooserWidgetCreators;
FObjectChooserWidgetFactories::FParameterMenuCreatorMap FObjectChooserWidgetFactories::ParameterMenuCreators;

void FObjectChooserWidgetFactories::RegisterWidgets()
{
}
	
}

#undef LOCTEXT_NAMESPACE
