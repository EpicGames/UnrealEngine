// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectClassColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "ClassViewerFilter.h"
#include "SPropertyAccessChainWidget.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectClassColumn.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "SClassViewer.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "SEnumCombo.h"

#define LOCTEXT_NAMESPACE "ObjectClassColumnEditor"

namespace UE::ChooserEditor
{
	class FBaseClassFilter : public IClassViewerFilter
	{
	public:
		const UClass* BaseClass = nullptr;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return BaseClass == nullptr || InClass->IsChildOf(BaseClass);
		};

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return BaseClass == nullptr || InUnloadedClassData->IsChildOf(BaseClass);
		};
	};

	static TSharedRef<SWidget> CreateObjectClassColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int32 Row)
	{
		FObjectClassColumn* ObjectClassColumn = static_cast<FObjectClassColumn*>(Column);

		if (Row == ColumnWidget_SpecialIndex_Fallback)
		{
			return SNullWidget::NullWidget;
		}
		else if (Row == ColumnWidget_SpecialIndex_Header)
		{
			const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
			const FText ColumnTooltip = LOCTEXT("Object Class Tooltip", "Object Class: cells pass if the Object input has a type that matches the cell's Class setting");
			const FText ColumnName = LOCTEXT("Object Class","Object Class");
			
			TSharedPtr<SWidget> DebugWidget = nullptr;
			if (Chooser->GetEnableDebugTesting())
			{
				UClass* AllowedClass = UObject::StaticClass();
				if (ObjectClassColumn->InputValue.IsValid())
				{
					const FChooserParameterObjectBase& Parameter = ObjectClassColumn->InputValue.Get<FChooserParameterObjectBase>();
					AllowedClass = Parameter.GetAllowedClass();
				}
				
				// create widget for test value object picker
				TSharedRef<SWidget> ObjectPicker = SNew(SObjectPropertyEntryBox)
															.ObjectPath_Lambda([ObjectClassColumn]() {
																return ObjectClassColumn->TestValue.ToString();
															})
															.OnObjectChanged_Lambda([ObjectClassColumn](const FAssetData& AssetData) {
																ObjectClassColumn->TestValue = AssetData.ToSoftObjectPath();
															})
															.AllowedClass(AllowedClass)
															.DisplayUseSelected(false)
															.DisplayBrowse(false)
															.DisplayThumbnail(false);
				
				ObjectPicker->SetEnabled(TAttribute<bool>::CreateLambda([Chooser]() { return !Chooser->HasDebugTarget(); }));
				DebugWidget = ObjectPicker;
			}
			return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
		}

		// create widget for cell
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SEnumComboBox, StaticEnum<EObjectClassColumnCellValueComparison>())
				.CurrentValue_Lambda([ObjectClassColumn, Row]()
				{
					if (ObjectClassColumn->RowValues.IsValidIndex(Row))
					{
						return static_cast<int32>(ObjectClassColumn->RowValues[Row].Comparison);
					}
					else
					{
						return 0;
					}
				})
				.OnEnumSelectionChanged_Lambda([ObjectClassColumn, Chooser, Row](int32 NewValue, ESelectInfo::Type SelectionType)
				{
					if (ObjectClassColumn->RowValues.IsValidIndex(Row))
					{
						FScopedTransaction ScopedTransaction(LOCTEXT("Change Comparison Type", "Change Comparison Type"));
						Chooser->Modify();
						ObjectClassColumn->RowValues[Row].Comparison = static_cast<EObjectClassColumnCellValueComparison>(NewValue);
					}
				})
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SComboButton)
				.OnGetMenuContent_Lambda([ObjectClassColumn, Chooser, Row]()
				{
					FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
					FClassViewerInitializationOptions Options;
					Options.Mode = EClassViewerMode::ClassPicker;

					TSharedRef<FBaseClassFilter> BaseClassFilter = MakeShared<FBaseClassFilter>();
					if (ObjectClassColumn->InputValue.IsValid())
					{
						const FChooserParameterObjectBase& Parameter = ObjectClassColumn->InputValue.Get<FChooserParameterObjectBase>();
						BaseClassFilter->BaseClass = Parameter.GetAllowedClass();
					}
					Options.ClassFilters.Add(BaseClassFilter);
					
					return ClassViewerModule.CreateClassViewer(
						Options,
						FOnClassPicked::CreateLambda([ObjectClassColumn, Chooser, Row](UClass* InNewClass)
						{
							if (ObjectClassColumn->RowValues.IsValidIndex(Row))
							{
								FScopedTransaction ScopedTransaction(LOCTEXT("Change Class", "Change Class"));
								Chooser->Modify();
								ObjectClassColumn->RowValues[Row].Value = InNewClass;
							}
							FSlateApplication::Get().DismissAllMenus();
						}));
				})
				.ButtonContent()
				[
					SNew(STextBlock).Text_Lambda([ObjectClassColumn, Row]()
					{
						if (ObjectClassColumn->RowValues.IsValidIndex(Row))
						{
							if (ObjectClassColumn->RowValues[Row].Value)
							{
								return FText::FromString(ObjectClassColumn->RowValues[Row].Value->GetName());
							}
						}
						return LOCTEXT("None","None");
					})
				]
		];
	}

	void RegisterObjectClassWidgets()
	{
		FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FObjectClassColumn::StaticStruct(), CreateObjectClassColumnWidget);
	}

} // namespace UE::ChooserEditor

#undef LOCTEXT_NAMESPACE
