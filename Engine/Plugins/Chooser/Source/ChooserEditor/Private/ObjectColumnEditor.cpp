// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "SPropertyAccessChainWidget.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectColumn.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "ObjectColumnEditor"

namespace UE::ChooserEditor
{
	static UClass* GetAllowedClass(const FObjectColumn* ObjectColumn)
	{
		UClass* AllowedClass = nullptr;
		if (const FChooserParameterObjectBase* InputValue = ObjectColumn->InputValue.GetPtr<FChooserParameterObjectBase>())
		{
			AllowedClass = InputValue->GetAllowedClass();
		}

		if (AllowedClass == nullptr)
		{
			AllowedClass = UObject::StaticClass();
		}

		return AllowedClass;
	}

	static TSharedRef<SObjectPropertyEntryBox> CreateObjectPicker(UObject* TransactionObject, FObjectColumn* ObjectColumn, int32 Row)
	{
		TSharedRef<SObjectPropertyEntryBox> ObjectPicker = SNew(SObjectPropertyEntryBox)
			.ObjectPath_Lambda([ObjectColumn, Row]() {
				return ObjectColumn->RowValues.IsValidIndex(Row) ? ObjectColumn->RowValues[Row].Value.ToString() : ObjectColumn->TestValue.ToString();
			})
			.AllowedClass(GetAllowedClass(ObjectColumn))
			.OnObjectChanged_Lambda([TransactionObject, ObjectColumn, Row](const FAssetData& AssetData) {
				if (ObjectColumn->RowValues.IsValidIndex(Row))
				{
					const FScopedTransaction Transaction(LOCTEXT("Edit RHS", "Edit Object Value"));
					TransactionObject->Modify(true);
					ObjectColumn->RowValues[Row].Value = AssetData.ToSoftObjectPath();
				}
				else
				{
					ObjectColumn->TestValue = AssetData.ToSoftObjectPath();
				}
			})
			.DisplayUseSelected(false)
			.DisplayBrowse(false)
			.DisplayThumbnail(false);

		if (Row >= 0)
		{
			ObjectPicker->SetVisibility(TAttribute<EVisibility>::CreateLambda([ObjectColumn, Row]() {
						return (ObjectColumn->RowValues.IsValidIndex(Row) &&
								ObjectColumn->RowValues[Row].Comparison == EObjectColumnCellValueComparison::MatchAny)
								   ? EVisibility::Collapsed
								   : EVisibility::Visible;
					}));
		}
		return ObjectPicker;
	}

	static TSharedRef<SWidget> CreateObjectColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int32 Row)
	{
		// Extend `SHorizontalBox` a little bit so we can poll for changes & recreate the object picker if necessary.
		class SHorizontalBoxEx : public SHorizontalBox
		{
			public:
			virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
			{
				SHorizontalBox::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

				UClass* CurrentAllowedClass = GetAllowedClass(ObjectColumn);
				if (AllowedClass != CurrentAllowedClass)
				{
					AllowedClass = CurrentAllowedClass;
					GetSlot(ObjectPickerSlot)[ObjectPickerFactory()];
				}
			}

			FObjectColumn* ObjectColumn = nullptr;
			UClass* AllowedClass = nullptr;
			TFunction<TSharedRef<SWidget>()> ObjectPickerFactory;
			int ObjectPickerSlot = 0;
		};

		FObjectColumn* ObjectColumn = static_cast<FObjectColumn*>(Column);

		if (Row == ColumnWidget_SpecialIndex_Fallback)
		{
			return SNullWidget::NullWidget;
		}
		else if (Row == ColumnWidget_SpecialIndex_Header)
		{
			const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.Filter");
			const FText ColumnTooltip = LOCTEXT("Object Tooltip", "Object: cells pass if the Object input is equal to the Object set in the cell");
			const FText ColumnName = LOCTEXT("Object","Object");
			
			TSharedPtr<SWidget> DebugWidget = nullptr;
			if (Chooser->GetEnableDebugTesting())
			{
				TSharedRef<SWidget> ObjectPicker = CreateObjectPicker(Chooser, ObjectColumn, Row);
				ObjectPicker->SetEnabled(TAttribute<bool>::CreateLambda([Chooser]() { return !Chooser->HasDebugTarget(); }));
				
				// create widget for test value object picker
				TSharedRef<SHorizontalBoxEx> CellWidget = SNew(SHorizontalBoxEx)
					+SHorizontalBox::Slot()
					.FillWidth(1)
					[
						ObjectPicker
					];
				
				CellWidget->SetCanTick(true);
				CellWidget->ObjectColumn = ObjectColumn;
				CellWidget->AllowedClass = GetAllowedClass(ObjectColumn);
				CellWidget->ObjectPickerFactory = [Chooser, ObjectColumn, Row]() { return CreateObjectPicker(Chooser, ObjectColumn, Row); };
				CellWidget->ObjectPickerSlot = 0;
				
				DebugWidget = CellWidget;
			}
			return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
		}


		// create widget for cell

		// clang-format off
		TSharedRef<SHorizontalBoxEx> CellWidget = SNew(SHorizontalBoxEx)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(55)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "FlatButton")
					.TextStyle(FAppStyle::Get(), "RichTextBlock.Bold")
					.HAlign(HAlign_Center)
					.Text_Lambda([ObjectColumn, Row]() {
						if (ObjectColumn->RowValues.IsValidIndex(Row))
						{
							switch (ObjectColumn->RowValues[Row].Comparison)
							{
								case EObjectColumnCellValueComparison::MatchEqual:
									return LOCTEXT("CompEqual", "=");

								case EObjectColumnCellValueComparison::MatchNotEqual:
									return LOCTEXT("CompNotEqual", "Not");

								case EObjectColumnCellValueComparison::MatchAny:
									return LOCTEXT("CompAny", "Any");
							}
						}
						return FText::GetEmpty();
					})
					.OnClicked_Lambda([Chooser, ObjectColumn, Row]() {
						if (ObjectColumn->RowValues.IsValidIndex(Row))
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit Comparison", "Edit Comparison Operation"));
							Chooser->Modify(true);
							// cycle through comparison options
							EObjectColumnCellValueComparison& Comparison = ObjectColumn->RowValues[Row].Comparison;
							const int32 NextComparison = (static_cast<int32>(Comparison) + 1) % static_cast<int32>(EObjectColumnCellValueComparison::Modulus);
							Comparison = static_cast<EObjectColumnCellValueComparison>(NextComparison);
						}
						return FReply::Handled();
					})
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				CreateObjectPicker(Chooser, ObjectColumn, Row)
			];
			// clang-format on
		
		CellWidget->SetCanTick(true);
		CellWidget->ObjectColumn = ObjectColumn;
		CellWidget->AllowedClass = GetAllowedClass(ObjectColumn);
		CellWidget->ObjectPickerFactory = [Chooser, ObjectColumn, Row]() { return CreateObjectPicker(Chooser, ObjectColumn, Row); };
		CellWidget->ObjectPickerSlot = 1;

		return CellWidget;
	}

	static TSharedRef<SWidget> CreateObjectPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
	{
		IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

		FObjectContextProperty* ContextProperty = reinterpret_cast<FObjectContextProperty*>(Value);

		return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(false).BindingColor("ObjectPinTypeColor").TypeFilter("object")
		.PropertyBindingValue(&ContextProperty->Binding)
		.OnValueChanged(ValueChanged);
	}

	void CreateObjectPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
    {
    	CreatePropertyAccessMenus<FObjectContextProperty>("object", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
     }


	void RegisterObjectWidgets()
	{
		FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterObjectBase::StaticStruct(), CreateObjectPropertyMenus);
		
		FObjectChooserWidgetFactories::RegisterWidgetCreator(FObjectContextProperty::StaticStruct(), CreateObjectPropertyWidget);
		FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FObjectColumn::StaticStruct(), CreateObjectColumnWidget);
	}

} // namespace UE::ChooserEditor

#undef LOCTEXT_NAMESPACE
