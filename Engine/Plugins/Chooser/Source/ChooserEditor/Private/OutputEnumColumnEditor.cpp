// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputEnumColumnEditor.h"
#include "EnumColumnEditor.h"
#include "OutputEnumColumn.h"
#include "SPropertyAccessChainWidget.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserColumnHeader.h"
#include "ChooserTableEditor.h"
#include "GraphEditorSettings.h"
#include "SEnumCombo.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Input/SButton.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "OutputEnumColumnEditor"

namespace UE::ChooserEditor
{

	
// Wrapper widget for EnumComboBox which will reconstruct the combo box when the Enum has changed
class SOutputEnumCell : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnValueSet, int);
	
	SLATE_BEGIN_ARGS(SOutputEnumCell)
	{}

	SLATE_ARGUMENT(UObject*, TransactionObject)
	SLATE_ARGUMENT(FOutputEnumColumn*, OutputEnumColumn)
	SLATE_ATTRIBUTE(int32, EnumValue);
	SLATE_EVENT(FOnValueSet, OnValueSet)
            
	SLATE_END_ARGS()

	TSharedRef<SWidget> CreateEnumComboBox()
	{
		if (const FOutputEnumColumn* OutputEnumColumnPointer = OutputEnumColumn)
		{
			if (OutputEnumColumnPointer->InputValue.IsValid())
			{
				if (const UEnum* Enum = OutputEnumColumnPointer->InputValue.template Get<FChooserParameterEnumBase>().GetEnum())
				{
					return SNew(SEnumComboBox, Enum)
						.bForceBitFlags(Enum->HasAnyEnumFlags(EEnumFlags::Flags) || Enum->HasMetaData(TEXT("Bitflags")))
						.IsEnabled_Lambda([this](){ return IsEnabled(); } )
						.CurrentValue(EnumValue)
						.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type)
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit RHS", "Edit Enum Value"));
							TransactionObject->Modify(true);
							OnValueSet.ExecuteIfBound(InEnumValue);
						});
				}
			}
		}
		
		return SNullWidget::NullWidget;
	}

	void UpdateEnumComboBox()
	{
		ChildSlot[ CreateEnumComboBox()	];
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (OutputEnumColumn)
		{
			const UEnum* CurrentEnumSource = nullptr;
			if (OutputEnumColumn->InputValue.IsValid())
			{
				CurrentEnumSource = OutputEnumColumn->InputValue.template Get<FChooserParameterEnumBase>().GetEnum(); 
			}
			if (EnumSource != CurrentEnumSource)
			{
				EnumComboBorder->SetContent(CreateEnumComboBox());
				EnumSource = CurrentEnumSource;
			}
		}
	}
    					

	void Construct( const FArguments& InArgs)
	{
		SetEnabled(InArgs._IsEnabled);
		
		SetCanTick(true);
		OutputEnumColumn = InArgs._OutputEnumColumn;
		TransactionObject = InArgs._TransactionObject;
		EnumValue = InArgs._EnumValue;
		OnValueSet = InArgs._OnValueSet;

		if (OutputEnumColumn)
		{
			if (OutputEnumColumn->InputValue.IsValid())
			{
				EnumSource = OutputEnumColumn->InputValue.template Get<FChooserParameterEnumBase>().GetEnum();
			}
		}

		UpdateEnumComboBox();

		int Row = RowIndex.Get();

		ChildSlot
		[
			SAssignNew(EnumComboBorder, SBorder).Padding(0).BorderBackgroundColor(FLinearColor(0,0,0,0))
			[
				CreateEnumComboBox()
			]
		];
		
	}

	~SOutputEnumCell()
	{
	}

private:
	UObject* TransactionObject = nullptr;
	FOutputEnumColumn* OutputEnumColumn = nullptr;
	const UEnum* EnumSource = nullptr;
	TSharedPtr<SBorder> EnumComboBorder;
	TAttribute<int> RowIndex;
	FDelegateHandle EnumChangedHandle;
	
	FOnValueSet OnValueSet;
	TAttribute<int32> EnumValue;
};

TSharedRef<SWidget> CreateOutputEnumColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FOutputEnumColumn* OutputEnumColumn = static_cast<FOutputEnumColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Enum Tooltip", "Output Enum:  writes the value from cell in the result row to the bound variable");
		const FText ColumnName = LOCTEXT("Output Enum","Output Enum");
	
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(STextBlock)
							.Text_Lambda([OutputEnumColumn]()
								{
									if (OutputEnumColumn)
									{
										if (const UEnum* Enum = OutputEnumColumn->GetEnum())
										{
											if (!Enum->HasAnyEnumFlags(EEnumFlags::Flags) && !Enum->HasMetaData(TEXT("Bitflags")))
											{
												return Enum->GetDisplayNameTextByValue(OutputEnumColumn->TestValue);
											}
			
											if (OutputEnumColumn->TestValue == 0)
											{
												return LOCTEXT("No flags set","(None)");
											} 
											return Enum->GetValueOrBitfieldAsDisplayNameText(OutputEnumColumn->TestValue);
										}
									}
									return FText();
								});
		}
		
		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}
	else if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNew(SOutputEnumCell).TransactionObject(Chooser).OutputEnumColumn(OutputEnumColumn)
       			.OnValueSet_Lambda([OutputEnumColumn](int Value)
       			{
       				OutputEnumColumn->FallbackValue.Value = static_cast<uint32>(Value);
       			})
       			.EnumValue_Lambda([OutputEnumColumn, Row]()
       			{
       				return OutputEnumColumn->FallbackValue.Value;
       			});
	}	

	// create cell widget
	
	return SNew(SOutputEnumCell).TransactionObject(Chooser).OutputEnumColumn(OutputEnumColumn)
			.OnValueSet_Lambda([OutputEnumColumn, Row](int Value)
			{
				if (OutputEnumColumn->RowValues.IsValidIndex(Row))
				{
					OutputEnumColumn->RowValues[Row].Value = static_cast<uint32>(Value);
				}
			})
			.EnumValue_Lambda([OutputEnumColumn, Row]()
			{
				return OutputEnumColumn->RowValues.IsValidIndex(Row) ? static_cast<int32>(OutputEnumColumn->RowValues[Row].Value) : 0;
			});
}

void RegisterOutputEnumWidgets()
{
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputEnumColumn::StaticStruct(), CreateOutputEnumColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
