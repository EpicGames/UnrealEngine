// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputFloatColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "OutputFloatColumn.h"
#include "ObjectChooserWidgetFactories.h"
#include "ChooserTableEditor.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Images/SImage.h"
#include "GraphEditorSettings.h"
#include "SPropertyAccessChainWidget.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "OutputBoolColumnEditor"

namespace UE::ChooserEditor
{
	
TSharedRef<SWidget> CreateOutputFloatColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FOutputFloatColumn* OutputFloatColumn = static_cast<FOutputFloatColumn*>(Column);

    if (Row == ColumnWidget_SpecialIndex_Header)
	{
    	// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Float Tooltip", "Output Float: writes the value from cell in the result row to the bound variable");
		const FText ColumnName = LOCTEXT("Output Float","Output Float");
        		
		TSharedPtr<SWidget> DebugWidget = nullptr;
		if (Chooser->GetEnableDebugTesting())
		{
			DebugWidget = SNew(SNumericEntryBox<float>)
				.IsEnabled(false)
				.Value_Lambda([OutputFloatColumn]() { return OutputFloatColumn->TestValue; });
		}
        
		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}
	if (Row == ColumnWidget_SpecialIndex_Fallback)
    {
		return SNew(SNumericEntryBox<double>)
        		.Value_Lambda([OutputFloatColumn]()
        		{
        			return OutputFloatColumn->FallbackValue;
        		})
        		.OnValueCommitted_Lambda([Chooser, OutputFloatColumn](double NewValue, ETextCommit::Type CommitType)
        		{
					const FScopedTransaction Transaction(LOCTEXT("Edit Float Value", "Edit Float Value"));
					Chooser->Modify(true);
					OutputFloatColumn->FallbackValue = NewValue;
        		});	
    }

	// create cell widget
	return SNew(SNumericEntryBox<double>)
    		.Value_Lambda([OutputFloatColumn, Row]()
    		{
    			return (OutputFloatColumn->RowValues.IsValidIndex(Row)) ? OutputFloatColumn->RowValues[Row] : 0;
    		})
    		.OnValueCommitted_Lambda([Chooser, OutputFloatColumn, Row](double NewValue, ETextCommit::Type CommitType)
    		{
    			if (OutputFloatColumn->RowValues.IsValidIndex(Row))
    			{
    				const FScopedTransaction Transaction(LOCTEXT("Edit Float Value", "Edit Float Value"));
    				Chooser->Modify(true);
    				OutputFloatColumn->RowValues[Row] = NewValue;
    			}
    		});
}

	
void RegisterOutputFloatWidgets()
{
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputFloatColumn::StaticStruct(), CreateOutputFloatColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
