// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputNameColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "SPropertyAccessChainWidget.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "OutputNameColumn.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "OutputNameColumnEditor"

namespace UE::ChooserEditor
{
	TSharedRef<SWidget> CreateOutputNameColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
	{
		FChooserOutputNameColumn* OutputNameColumn = static_cast<FChooserOutputNameColumn*>(Column);

		if (Row == ColumnWidget_SpecialIndex_Header)
		{
			// create column header widget
			const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
			const FText ColumnTooltip = LOCTEXT("Output Name Tooltip", "Output Name: writes the value from cell in the result row to the bound variable");
			const FText ColumnName = LOCTEXT("Output Name","Output Name");
					
			TSharedPtr<SWidget> DebugWidget = nullptr;
			if (Chooser->GetEnableDebugTesting())
			{
				DebugWidget = SNew(SEditableTextBox)
					.IsEnabled(false)
					.Text_Lambda([OutputNameColumn]() { return FText::FromName(OutputNameColumn->TestValue); });
			}
			
			return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
		}
		if (Row == ColumnWidget_SpecialIndex_Fallback)
		{
			return SNew(SEditableTextBox)
					.Text_Lambda([OutputNameColumn]()
					{
						return FText::FromName(OutputNameColumn->FallbackValue);
					})
					.OnTextCommitted_Lambda([Chooser, OutputNameColumn](const FText& NewValue, ETextCommit::Type CommitType)
					{
						const FScopedTransaction Transaction(LOCTEXT("Edit Name Value", "Edit Name Value"));
						Chooser->Modify(true);
						OutputNameColumn->FallbackValue = FName(NewValue.ToString());
					});	
		}

		return SNew(SEditableTextBox)
				.Text_Lambda([OutputNameColumn, Row]()
				{
					return OutputNameColumn->RowValues.IsValidIndex(Row) ? FText::FromName(OutputNameColumn->RowValues[Row]) : FText();
				})
				.OnTextCommitted_Lambda([Chooser, OutputNameColumn, Row](const FText& NewValue, ETextCommit::Type CommitType)
				{
					const FScopedTransaction Transaction(LOCTEXT("Edit Name", "Edit Name"));
					Chooser->Modify(true);
					if (OutputNameColumn->RowValues.IsValidIndex(Row))
					{
						OutputNameColumn->RowValues[Row] = FName(NewValue.ToString());
					}
				});
	}

	void RegisterOutputNameWidgets()
	{
		FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FChooserOutputNameColumn::StaticStruct(), CreateOutputNameColumnWidget);
	}

} // namespace UE::ChooserEditor

#undef LOCTEXT_NAMESPACE
