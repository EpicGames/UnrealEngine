// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputObjectColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "SPropertyAccessChainWidget.h"
#include "GraphEditorSettings.h"
#include "ObjectChooserWidgetFactories.h"
#include "OutputObjectColumn.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/TransactionCommon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "OutputObjectColumnEditor"

namespace UE::ChooserEditor
{
	static UClass* GetAllowedClass(const FOutputObjectColumn* OutputObjectColumn)
	{
		UClass* AllowedClass = nullptr;
		if (const FChooserParameterObjectBase* InputValue = OutputObjectColumn->InputValue.GetPtr<FChooserParameterObjectBase>())
		{
			AllowedClass = InputValue->GetAllowedClass();
		}

		if (AllowedClass == nullptr)
		{
			AllowedClass = UObject::StaticClass();
		}

		return AllowedClass;
	}


	
	// Wrapper widget for EnumComboBox which will reconstruct the combo box when the Enum has changed
    class SOutputObjectCell : public SCompoundWidget
    {
    public:
    
    	DECLARE_DELEGATE_OneParam(FOnValueSet, int);
    	
    	SLATE_BEGIN_ARGS(SOutputObjectCell)
    	{}
    
    	SLATE_ARGUMENT(UChooserTable*, Chooser)
    	SLATE_ARGUMENT(FOutputObjectColumn*, Column)
    	SLATE_ARGUMENT(IChooserTableWidgetInterface*, ChooserWidgetInterface)
    	SLATE_ATTRIBUTE(int32, Row)
                
    	SLATE_END_ARGS()
    
    	TSharedRef<SWidget> CreateWidget()
    	{
    		if (Column)
    		{
				int Row = RowIndex.Get();
				UChooserTable* ContextOwner = Chooser->GetRootChooser();

				if (Row == ColumnWidget_SpecialIndex_Fallback)
				{
					UClass* AllowedClass = UObject::StaticClass();
					if (Column->InputValue.IsValid())
					{
						AllowedClass = Column->InputValue.Get<FChooserParameterObjectBase>().GetAllowedClass();
					}
					
					TSharedPtr<SWidget> ResultWidget = FObjectChooserWidgetFactories::CreateWidget(false, ContextOwner, FObjectChooserBase::StaticStruct(),
						&Column->FallbackValue.Value, AllowedClass, FChooserWidgetValueChanged(), ChooserWidgetInterface, LOCTEXT("None", "(None)"));
					return ResultWidget. ToSharedRef();
				}
				else if (Row == ColumnWidget_SpecialIndex_Header)
				{
					// create column header widget
					const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
					const FText ColumnTooltip = LOCTEXT("Output Object Tooltip", "Output Object: writes the value from cell in the result row to the bound variable");
					const FText ColumnName = LOCTEXT("Output Object","Output Object");
							
					TSharedPtr<SWidget> DebugWidget = nullptr;
					
					return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
				}
 
				if (Column->RowValues.IsValidIndex(Row))
				{
					UClass* AllowedClass = UObject::StaticClass();
					if (Column->InputValue.IsValid())
					{
						AllowedClass = Column->InputValue.Get<FChooserParameterObjectBase>().GetAllowedClass();
					}
					
					TSharedPtr<SWidget> ResultWidget = FObjectChooserWidgetFactories::CreateWidget(false, ContextOwner, FObjectChooserBase::StaticStruct(),
						&Column->RowValues[Row].Value, AllowedClass, FChooserWidgetValueChanged(), ChooserWidgetInterface, LOCTEXT("None", "(None)"));
					return ResultWidget. ToSharedRef();
				}
 
				return SNullWidget::NullWidget;
    				
    		}
    		
    		return SNullWidget::NullWidget;
    	}
    
    	void UpdateWidget()
    	{
    		ChildSlot[ CreateWidget() ];
    	}
    
    	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
    	{
    		UClass* CurrentBaseType = nullptr;
    		if (Column->InputValue.IsValid())
    		{
    			CurrentBaseType = Column->InputValue.Get<FChooserParameterObjectBase>().GetAllowedClass();
    		}
    		if (ObjectBaseType != CurrentBaseType)
    		{
    			UpdateWidget();
    			ObjectBaseType = CurrentBaseType;
    		}
    	}
        					
    
    	void Construct( const FArguments& InArgs)
    	{
    		SetEnabled(InArgs._IsEnabled);
    		
    		SetCanTick(true);
    		Column = InArgs._Column;
    		Chooser = InArgs._Chooser;
    		RowIndex = InArgs._Row;
    		ChooserWidgetInterface = InArgs._ChooserWidgetInterface;
    
			if (Column)
    		{
    			if (Column->InputValue.IsValid())
    			{
    				ObjectBaseType = Column->InputValue.Get<FChooserParameterObjectBase>().GetAllowedClass();
    			}
    		}
    
    		UpdateWidget();
    	}
    
    	virtual ~SOutputObjectCell()
    	{
    	}
    
    private:
    	UChooserTable* Chooser = nullptr;
    	FOutputObjectColumn* Column = nullptr;
		UClass* ObjectBaseType = nullptr;
		IChooserTableWidgetInterface* ChooserWidgetInterface = nullptr;
    	TAttribute<int> RowIndex;
		
    };


	static TSharedRef<SWidget> CreateOutputObjectColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int32 Row)
	{
		return SNew(SOutputObjectCell).Chooser(Chooser).Column(static_cast<FOutputObjectColumn*>(Column)).Row(Row);
	}

	void RegisterOutputObjectWidgets()
	{
		FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputObjectColumn::StaticStruct(), CreateOutputObjectColumnWidget);
	}

} // namespace UE::ChooserEditor

#undef LOCTEXT_NAMESPACE
