// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputStructColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "OutputStructColumn.h"
#include "SPropertyAccessChainWidget.h"
#include "ObjectChooserWidgetFactories.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "GraphEditorSettings.h"

#define LOCTEXT_NAMESPACE "StructOutputColumnEditor"

namespace UE::ChooserEditor
{
	
TSharedRef<SWidget> CreateOutputStructColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
    	// create column header widget
		const FSlateBrush* ColumnIcon = FCoreStyle::Get().GetBrush("Icons.ArrowRight");
		const FText ColumnTooltip = LOCTEXT("Output Struct Tooltip", "Output Struct: writes the value from cell in the result row to the bound variable");
		const FText ColumnName = LOCTEXT("Output Struct","Output Struct");
        		
		TSharedPtr<SWidget> DebugWidget = nullptr;
        
		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget,
				FChooserWidgetValueChanged::CreateLambda([Column]()
				{
					FOutputStructColumn* StructColumn = static_cast<FOutputStructColumn*>(Column);
					StructColumn->StructTypeChanged();
				})	
			);
	}
	
	FOutputStructColumn* StructColumn = static_cast<FOutputStructColumn*>(Column);

	TAttribute<FText> StructValueAttribute = MakeAttributeLambda([StructColumn, Row]()
		{
			const FInstancedStruct* RowValue = nullptr;
			if (Row == ColumnWidget_SpecialIndex_Fallback)
			{
				RowValue = &StructColumn->FallbackValue;
			}
			else
			{
				if (StructColumn->RowValues.IsValidIndex(Row))
				{
					RowValue = &StructColumn->RowValues[Row];
				}
			}

			FString Value;
			if (RowValue)
			{
				if (const UScriptStruct* ScriptStruct = RowValue->GetScriptStruct())
				{
					void* DefaultStructMemory = FMemory_Alloca_Aligned(ScriptStruct->GetStructureSize(), ScriptStruct->GetMinAlignment());
					ScriptStruct->InitializeStruct(DefaultStructMemory);
					ScriptStruct->ExportText(Value, RowValue->GetMemory(), DefaultStructMemory, nullptr, PPF_ExternalEditor, nullptr);
					ScriptStruct->DestroyStruct(DefaultStructMemory);
					
					return FText::FromString(Value);
				}
			}
			
			return FText::FromString(TEXT("()"));
		});

	TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
		.Text(StructValueAttribute);
	TextBlock->SetToolTipText(StructValueAttribute);

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			TextBlock
		];
}

TSharedRef<SWidget> CreateStructPropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FStructContextProperty* ContextProperty = reinterpret_cast<FStructContextProperty*>(Value);

	return SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).BindingColor("StructPinTypeColor").TypeFilter("struct")
		.PropertyBindingValue(&ContextProperty->Binding)
		.OnValueChanged(ValueChanged);
}

void CreateStructPropertyMenus(UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)
{
	CreatePropertyAccessMenus<FStructContextProperty>("struct", TransactionObject, ContextClassOwner, Parameter, MenuBuilder, BindingChanged);
}
	
void RegisterStructWidgets()
{
	FObjectChooserWidgetFactories::RegisterParameterMenuCreator(FChooserParameterStructBase::StaticStruct(), CreateStructPropertyMenus);
	FObjectChooserWidgetFactories::RegisterWidgetCreator(FStructContextProperty::StaticStruct(), CreateStructPropertyWidget);
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FOutputStructColumn::StaticStruct(), CreateOutputStructColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
