// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAccessChainCustomization.h"

#include "Chooser.h"
#include "ChooserPropertyAccess.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "GraphEditorSettings.h"
#include "SClassViewer.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyAccessEditor.h"
#include "ScopedTransaction.h"
#include "SPropertyAccessChainWidget.h"

#define LOCTEXT_NAMESPACE "PropertyAccessChainCustomization"

namespace UE::ChooserEditor
{
	
void FPropertyAccessChainCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FProperty* Property = PropertyHandle->GetProperty();

	UClass* ContextClass = nullptr;
	static FName TypeMetaData = "BindingType";
	static FName ColorMetaData = "BindingColor";
	static FName AllowFunctionsMetaData = "BindingAllowFunctions";

	FString TypeFilter = PropertyHandle->GetMetaData(TypeMetaData);
	FString BindingColor = PropertyHandle->GetMetaData(ColorMetaData);
	FString BindingAllowFunctions = PropertyHandle->GetMetaData(AllowFunctionsMetaData);
	bool AllowFunctions = BindingAllowFunctions.ToBool();
		

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	UObject* OuterObject = OuterObjects[0];
	
	while (OuterObject && !OuterObject->Implements<UHasContextClass>())
	{
		OuterObject = OuterObject->GetOuter();
	}
	
	IHasContextClass* HasContext = nullptr;
	if (OuterObject)
	{
		HasContext = static_cast<IHasContextClass*>(OuterObject->GetInterfaceAddress(UHasContextClass::StaticClass()));
	}

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SPropertyAccessChainWidget).ContextClassOwner(HasContext).TypeFilter(TypeFilter).BindingColor(BindingColor).AllowFunctions(AllowFunctions)
		.PropertyBindingValue_Lambda([PropertyHandle]()
		{
			void* data;
			PropertyHandle->GetValueData(data);
			return reinterpret_cast<FChooserPropertyBinding*>(data);
		})
		.OnAddBinding_Lambda([PropertyHandle, this, HasContext, TypeFilter](FName /*InPropertyName*/, const TArray<FBindingChainElement>& InBindingChain)
		{

			UObject* TransactionObject = Cast<UObject>(HasContext);
            	
            const FScopedTransaction Transaction(NSLOCTEXT("PropertyAccessChainCustomization", "Set Binding", "Set Binding"));
   
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);
			for (UObject* OuterObject : OuterObjects)
			{
				OuterObject->Modify();				
			}
			
			
			if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
			{
				if (StructProperty->Struct->IsChildOf(FChooserPropertyBinding::StaticStruct()))
				{
					TArray<void*> PropertyDatas;
					PropertyHandle->AccessRawData(PropertyDatas);
					for (void* PropertyData : PropertyDatas)
					{
						SPropertyAccessChainWidget::SetPropertyBinding(HasContext, InBindingChain, *static_cast<FChooserPropertyBinding*>(PropertyData));
					}
				}
			}
			
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		})
	];
}
	
void FPropertyAccessChainCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE