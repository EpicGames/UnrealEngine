// Copyright Epic Games, Inc. All Rights Reserved.

#include "RandomizeColumnEditor.h"

#include <ChooserColumnHeader.h>

#include "RandomizeColumn.h"
#include "SPropertyAccessChainWidget.h"
#include "ObjectChooserWidgetFactories.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "GraphEditorSettings.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "RandomizeColumnEditor"

namespace UE::ChooserEditor
{

TSharedRef<SWidget> CreateRandomizeColumnWidget(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)
{
	FRandomizeColumn* RandomizeColumn = static_cast<FRandomizeColumn*>(Column);
	
	if (Row == ColumnWidget_SpecialIndex_Fallback)
	{
		return SNullWidget::NullWidget;
	}
	if (Row == ColumnWidget_SpecialIndex_Header)
	{
		// create column header widget
		const FSlateBrush* ColumnIcon = FAppStyle::Get().GetBrush("Icons.Help");
		const FText ColumnTooltip = LOCTEXT("Randomize Tooltip", "Randomize: randomly selects a single result from the rows which passed all other columns, or the rows with equal, minimum cost, for cost based columns.  Optional Randomization Context variable binding can be used to reduce (or eliminate) the probability of selecting the same entry twice in a row.");
		const FText ColumnName = LOCTEXT("Randomize","Randomize");
        		
		TSharedPtr<SWidget> DebugWidget = nullptr;

		return MakeColumnHeaderWidget(Chooser, Column, ColumnName, ColumnTooltip, ColumnIcon, DebugWidget);
	}

	// create cell widget
	return
	SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().FillWidth(1)
		+ SHorizontalBox::Slot().AutoWidth()
		[
			SNew(SBox).WidthOverride(75).Content()
			[
				SNew(SNumericEntryBox<float>)
    				.Value_Lambda([RandomizeColumn, Row]()
    				{
    					if (!RandomizeColumn->RowValues.IsValidIndex(Row))
    					{
    						return 0.0f;
    					}
    					return RandomizeColumn->RowValues[Row];
    				})
    				.OnValueCommitted_Lambda([Chooser, Row, RandomizeColumn](float Value, ETextCommit::Type CommitType)
    				{
    					if (RandomizeColumn->RowValues.IsValidIndex(Row))
    					{
    						const FScopedTransaction Transaction(LOCTEXT("Edit Randomize Cell Data", "Edit Randomize Cell Data"));
    						Chooser->Modify(true);
    						RandomizeColumn->RowValues[Row] = FMath::Max(Value, 0.0);
    					}
    				})
    		]
    	]
		+ SHorizontalBox::Slot().FillWidth(1);
}
	
void RegisterRandomizeWidgets()
{
	FObjectChooserWidgetFactories::RegisterColumnWidgetCreator(FRandomizeColumn::StaticStruct(), CreateRandomizeColumnWidget);
}
	
}

#undef LOCTEXT_NAMESPACE
