// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerChooser.h"

#include "Chooser.h"
#include "ChooserProvider.h"
#include "Common/ProviderLock.h"
#include "GameFramework/Actor.h"
#include "IGameplayProvider.h"
#include "TraceServices/Model/Frames.h"

FRewindDebuggerChooser::FRewindDebuggerChooser()
{

}

void FRewindDebuggerChooser::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FChooserProvider* ChooserProvider = AnalysisSession->ReadProvider<FChooserProvider>(FChooserProvider::ProviderName);
	const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
	if (!ChooserProvider || !GameplayProvider)
	{
		return;
	}

	const float CurrentTraceTime = IRewindDebugger::Instance()->CurrentTraceTime();

	bool bFrameFound;
	TraceServices::FFrame Frame;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, Frame);
	}

	if (bFrameFound)
	{
		TraceServices::FProviderReadScopeLock ChooserProviderReadScope(*ChooserProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
		ChooserProvider->EnumerateChooserEvaluationTimelines([GameplayProvider, ChooserProvider, RewindDebugger, Frame](uint64 OwnerId, const FChooserProvider::ChooserEvaluationTimeline& ChooserEvaluationTimeline)
		{
			double StartTime = RewindDebugger->GetScrubTime();
			double EndTime = StartTime + Frame.EndTime - Frame.StartTime;
			
			ChooserEvaluationTimeline.EnumerateEvents(StartTime, EndTime, [RewindDebugger, GameplayProvider, ChooserProvider, StartTime, EndTime, OwnerId](double InStartTime, double InEndTime, uint32 InDepth, const FChooserEvaluationData& ChooserEvaluationData)  
			{
				const FObjectInfo& ChooserInfo = GameplayProvider->GetObjectInfo(ChooserEvaluationData.ChooserId);
				
				if (UChooserTable* Chooser = FindObject<UChooserTable>(nullptr, ChooserInfo.PathName))
				{
					const FObjectInfo& ContextObjectInfo = GameplayProvider->GetObjectInfo(OwnerId);
					FString DebugName = ContextObjectInfo.Name;

					if (const FObjectInfo* ActorInfo = RewindDebugger->FindTypedOuterInfo<AActor>(GameplayProvider, OwnerId))
					{
						DebugName += " in " + FString(ActorInfo->Name);
					}

					if (const FWorldInfo* WorldInfo = GameplayProvider->FindWorldInfoFromObject(OwnerId))
					{
						const FObjectInfo& WorldObjectInfo = GameplayProvider->GetObjectInfo(WorldInfo->Id);
						
						if (WorldInfo->Type == FWorldInfo::EType::EditorPreview)
						{
							DebugName += " (Preview)";
						}
						else if (WorldInfo->NetMode == FWorldInfo::ENetMode::DedicatedServer)
						{
							DebugName += " (Server)";
						}
						else if (WorldInfo->NetMode == FWorldInfo::ENetMode::Client)
						{
							DebugName += FString(" (Client ") + FString::FromInt(WorldInfo->PIEInstanceId) + ")";
						}
					}
					
					// add to recent context objects list, so that this object is selectable as a target in the chooser editor
                 	Chooser->AddRecentContextObject(DebugName);
					
					UChooserTable* RootChooser = Chooser->GetRootChooser();
					if (RootChooser->HasDebugTarget())
					{
						if (RootChooser->GetDebugTargetName() == DebugName)
						{
							Chooser->SetDebugSelectedRows(ChooserEvaluationData.SelectedIndices);
							Chooser->SetDebugTestValuesValid(true);

							ChooserProvider->ReadChooserValueTimeline(OwnerId, [StartTime, EndTime, Chooser, &ChooserEvaluationData](const FChooserProvider::ChooserValueTimeline& ValueTimeline)
							{
								ValueTimeline.EnumerateEvents(StartTime,EndTime,[Chooser, ChooserEvaluationData](double StartTime, double EndTime, uint32 Depth, const FChooserValueData& ValueData)
								{
									if (ChooserEvaluationData.ChooserId == ValueData.ChooserId)
									{
										for(FInstancedStruct& ColumnStruct : Chooser->ColumnsStructs)
										{
											FChooserColumnBase& Column = ColumnStruct.GetMutable<FChooserColumnBase>();
											if (const FChooserParameterBase* InputValue = Column.GetInputValue())
											{
												if (ValueData.Key == InputValue->GetDebugName())
												{
													Column.SetTestValue(ValueData.Value);
												}
											}
										}
									}
									return TraceServices::EEventEnumerate::Continue;
								});
							});
						}
					}
				}
				
				return TraceServices::EEventEnumerate::Continue;
			});
			

		});
	}
}