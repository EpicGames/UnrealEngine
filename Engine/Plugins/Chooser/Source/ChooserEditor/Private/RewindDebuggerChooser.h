// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerExtension.h"

// Rewind debugger extension for Chooser support

class FRewindDebuggerChooser : public IRewindDebuggerExtension
{
public:
	FRewindDebuggerChooser();
	virtual ~FRewindDebuggerChooser() = default;

	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual FString GetName() { return TEXT("ChooserDebugger"); }
};
