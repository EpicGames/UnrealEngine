// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserColumnHandle.h"
#include "ChooserTableEditor.h"
#include "RandomizeColumn.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "ChooserColumnHandle"

namespace UE::ChooserEditor
{

	TSharedRef<FChooserColumnDragDropOp> FChooserColumnDragDropOp::New(TSharedPtr<FChooserTableViewModel> InViewModel, uint32 InColumnIndex)
	{
		TSharedRef<FChooserColumnDragDropOp> Operation = MakeShareable(new FChooserColumnDragDropOp());
		Operation->ChooserViewModel = InViewModel;
		Operation->ColumnIndex = InColumnIndex;
		Operation->DefaultHoverText = LOCTEXT("Chooser Column", "Chooser Column");
		Operation->CurrentHoverText = Operation->DefaultHoverText;
		
		Operation->Construct();

		return Operation;
	}
	
	void SChooserColumnHandle::Construct(const FArguments& InArgs)
	{
		TableHasFocus = InArgs._TableHasFocus;
		ChooserViewModel = InArgs._ViewModel;
		check(ChooserViewModel);
		ColumnIndex = InArgs._ColumnIndex;
		bNoDropAfter = InArgs._NoDropAfter;

		ChildSlot
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock).Color_Lambda([this](){ return TableHasFocus.Get() ? FStyleColors::Select.GetSpecifiedColor() : FStyleColors::SelectInactive.GetSpecifiedColor(); } )
							.Visibility_Lambda(
							[this]()
							{
								if (ChooserViewModel->IsColumnSelected(ColumnIndex))
								{
									return EVisibility::Visible;
								}
								return EVisibility::Hidden;
							})
				]
				+ SOverlay::Slot()
				[
					InArgs._Content.Widget
				]
				+ SOverlay::Slot().HAlign(HAlign_Right)
				[
					SNew(SSeparator).Orientation(Orient_Vertical)
					.SeparatorImage(FAppStyle::GetBrush("PropertyEditor.VerticalDottedLine"))
					.ColorAndOpacity_Lambda([this]() { return FSlateColor(bDropSupported ? EStyleColor::Select : EStyleColor::Error); })
					.Visibility_Lambda([this]() { return bDragActive && !bDropBefore ? EVisibility::Visible : EVisibility::Hidden; })
				]
				+ SOverlay::Slot().HAlign(HAlign_Left)
				[
					SNew(SSeparator).Orientation(Orient_Vertical)
					.SeparatorImage(FAppStyle::GetBrush("PropertyEditor.VerticalDottedLine"))
					.ColorAndOpacity_Lambda([this]() { return FSlateColor(bDropSupported ? EStyleColor::Select : EStyleColor::Error); }) 
					.Visibility_Lambda([this]() { return bDragActive && bDropBefore ? EVisibility::Visible : EVisibility::Hidden; })
				]
		];
	}
	
	FReply SChooserColumnHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		ChooserViewModel->SelectColumn(ChooserViewModel->GetChooser(), ColumnIndex);
		
		// act as a move handle if the row is already selected, and if there are multiselect modifiers pressed
		if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton && !MouseEvent.IsControlDown() && !MouseEvent.IsShiftDown()
			&& ChooserViewModel->IsColumnSelected(ColumnIndex))
		{
			UChooserTable* Chooser = ChooserViewModel->GetChooser();
			if (!Chooser->ColumnsStructs[ColumnIndex].GetPtr<FRandomizeColumn>()) // don't allow dragging the Randomize Column
			{
				return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
			}
		}
		
		return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	};

	FReply SChooserColumnHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		// clear row selection so that delete key can't cause the selected row to be deleted
		ChooserViewModel->ClearSelectedColumn();
		
		TSharedRef<FChooserColumnDragDropOp> DragDropOp = FChooserColumnDragDropOp::New(ChooserViewModel, ColumnIndex);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}


	void SChooserColumnHandle::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		bDropSupported = false;

		UChooserTable* Chooser = ChooserViewModel->GetChooser();
		
		if (!Chooser->ColumnsStructs.IsValidIndex(ColumnIndex))
		{
			return;
		}

		if (TSharedPtr<FChooserColumnDragDropOp> Operation = DragDropEvent.GetOperationAs<FChooserColumnDragDropOp>())
		{
			bDropSupported = true;
		}
		
		float Center = MyGeometry.Position.X + MyGeometry.Size.X;
		bDropBefore = bNoDropAfter || DragDropEvent.GetScreenSpacePosition().X < Center;
		bDragActive = true;
	}
	void SChooserColumnHandle::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		bDragActive = false;
	}

	FReply SChooserColumnHandle::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		DragActiveCounter = 2;
		bDragActive = true;
		float Center = MyGeometry.AbsolutePosition.X + MyGeometry.Size.X/2;
		bDropBefore = bNoDropAfter || DragDropEvent.GetScreenSpacePosition().X < Center;
		return FReply::Handled();
	}

	FReply SChooserColumnHandle::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		bDragActive = false;
		
		if (!bDropSupported)
		{
			return FReply::Unhandled();
		}
		
		if (TSharedPtr<FChooserColumnDragDropOp> Operation = DragDropEvent.GetOperationAs<FChooserColumnDragDropOp>())
		{
			if (UChooserTable* Chooser = Operation->ChooserViewModel->GetChooser())
			{
				int NewColumnIndex = 0; 
				if (bDropBefore)
				{
					NewColumnIndex = ChooserViewModel->MoveColumn(Operation->ColumnIndex, ColumnIndex);
				}
				else
				{
					NewColumnIndex = ChooserViewModel->MoveColumn(Operation->ColumnIndex, ColumnIndex+1);
				}

				if (NewColumnIndex >= 0)
				{
					ChooserViewModel->SelectColumn(Chooser, NewColumnIndex);
				}
			}
		}
				
		return FReply::Handled();		
	}

}

#undef LOCTEXT_NAMESPACE
