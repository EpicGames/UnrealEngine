// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

namespace UE::ChooserEditor
{

class FChooserTableViewModel;

class FChooserColumnDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FChooserColumnDragDropOp, FDecoratedDragDropOp)

	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	uint32 ColumnIndex;

	/** Constructs the drag drop operation */
	static TSharedRef<FChooserColumnDragDropOp> New(TSharedPtr<FChooserTableViewModel> InViewModel, uint32 InColumnIndex);
};

class SChooserColumnHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChooserColumnHandle)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_ARGUMENT(TSharedPtr<FChooserTableViewModel>, ViewModel)
	SLATE_ARGUMENT(uint32, ColumnIndex)
	SLATE_ARGUMENT(bool, NoDropAfter)
	SLATE_ATTRIBUTE(bool, TableHasFocus);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	void OnDragLeave(const FDragDropEvent& DragDropEvent);
	FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

private:
	TAttribute<bool> TableHasFocus;
	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	int32 ColumnIndex = 0;
	int32 DragActiveCounter = 0;
	bool bDragActive = false;
	bool bDropBefore = false;
	bool bDropSupported = false;
	bool bNoDropAfter = false;
};

}
