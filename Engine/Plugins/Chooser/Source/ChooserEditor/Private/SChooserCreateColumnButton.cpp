// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserCreateColumnButton.h"
#include "ChooserTableEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "SPositiveActionButton.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "AddColumnButton"

namespace UE::ChooserEditor
{
	struct FColumnTypeInfoStruct
	{
		int SortOrder = 100;
		FString Category;
		UScriptStruct* Type;
		bool operator < (const FColumnTypeInfoStruct& Other) const
		{
			if (Category == Other.Category)
			{
				return Type->GetDisplayNameText().ToString() < Other.Type->GetDisplayNameText().ToString();
			}
			else if (SortOrder == Other.SortOrder)
			{
				return Category < Other.Category;
			}
			else
			{
				return SortOrder < Other.SortOrder;
			}
		}
	};

	TSharedRef<SWidget>	SChooserCreateColumnButton::MakeCreateColumnMenu()
    {
    	FMenuBuilder MenuBuilder(true, nullptr);
    	static TArray<FColumnTypeInfoStruct> ColumnTypes;
    
    	if (ColumnTypes.IsEmpty())
    	{
    		UScriptStruct* BaseType = FChooserColumnBase::StaticStruct();
    		for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
    		{
    			if (*StructIt != BaseType && StructIt->IsChildOf(BaseType))
    			{
    				if (!StructIt->HasMetaData("Hidden"))
    				{
    					FColumnTypeInfoStruct Info;
    					Info.Type = *StructIt;
    					Info.Category = StructIt->HasMetaData("Category") ? StructIt->GetMetaData("Category") : "Other";
    
    					if (Info.Category == "Filter")
    					{
    						Info.SortOrder = 1;
    					}
    					else if (Info.Category == "Scoring")
                       	{
                       		Info.SortOrder = 2;
                       	}
    					else if (Info.Category == "Output")
                       	{
                       		Info.SortOrder = 3;
                       	}
    					else if (Info.Category == "Random")
    					{
    						Info.SortOrder = 4;
    					}
    
    					ColumnTypes.Add(Info);
    				}
    			}
    		}
    		ColumnTypes.Sort();
    	}
    
    	FString Section = "";
    	for(FColumnTypeInfoStruct& Type : ColumnTypes)
    	{
    		if (Section != Type.Category)
    		{
    			if (Section != "")
    			{
    				MenuBuilder.EndSection();
    			}
    			Section = Type.Category;
    			MenuBuilder.BeginSection(FName(Section), FText::FromString(Section));
    		}
    		
    		MenuBuilder.AddMenuEntry(Type.Type->GetDisplayNameText(), Type.Type->GetToolTipText(), FSlateIcon(),
    			FUIAction(
    				FExecuteAction::CreateLambda([this, Type]()
    				{
    					ChooserViewModel->AddColumn(Type.Type);
    				}))
    			);
    							
    	}
    	return MenuBuilder.MakeWidget();
    }
		
	void SChooserCreateColumnButton::Construct(const FArguments& InArgs)
	{
		ChooserViewModel = InArgs._ViewModel;
		check(ChooserViewModel);
		
		ChildSlot
		[
			SNew(SPositiveActionButton)
            	.Text(LOCTEXT("Add Column", "Add Column"))
            	.OnGetMenuContent(this, &SChooserCreateColumnButton::MakeCreateColumnMenu)
		];

	}

	SChooserCreateColumnButton::~SChooserCreateColumnButton()
	{
	}

}

#undef LOCTEXT_NAMESPACE
