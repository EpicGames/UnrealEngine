// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserCreateDialog.h"
#include "Chooser.h"
#include "ChooserInitializer.h"
#include "ChooserEditorSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Editor.h"
#include "ObjectChooserClassFilter.h"
#include "StructViewerModule.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "CreateChooserDialog"

/** Constructs this widget with InArgs */
void SChooserCreateDialog::Construct( const FArguments& InArgs )
{
	bOkClicked = false;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	// DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowLooseProperties = true;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject(ChooserFactory.Get());

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot().FillHeight(1)
		[
			DetailsView.ToSharedRef()
		]

		// Ok/Cancel buttons
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(10.0f)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0,0)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("CreateAnimBlueprintCreate_Tooltip", "Create a new Chooser Table Asset.."))
				.IsEnabled_Lambda([this]()
				{
					return true;
				})
				.HAlign(HAlign_Center)
				.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
				.OnClicked(this, &SChooserCreateDialog::OkClicked)
				.IsEnabled_Lambda([this]() { return ChooserFactory->ChooserInitializer.IsValid(); } )
				.Text(LOCTEXT("Create", "Create"))
			]
			+SUniformGridPanel::Slot(1,0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
				.OnClicked(this, &SChooserCreateDialog::CancelClicked)
				.Text(LOCTEXT("Cancel", "Cancel"))
			]
		]
	];
}

/** Sets properties for the supplied AnimBlueprintFactory */
bool SChooserCreateDialog::ConfigureProperties(TWeakObjectPtr<UChooserSignatureFactory> InChooserFactory)
{
	ChooserFactory = InChooserFactory;
	if (!UChooserEditorSettings::Get().DefaultCreateType.IsEmpty())
	{
		FTopLevelAssetPath StructPath(UChooserEditorSettings::Get().DefaultCreateType);
		if (UScriptStruct* DefaultInitializer = FindObject<UScriptStruct>(StructPath))
		{
			ChooserFactory->ChooserInitializer.InitializeAs(DefaultInitializer);
		}
	}
	
	DetailsView->SetObject(ChooserFactory.Get());
	
	const float AppScale = FSlateApplication::Get().GetApplicationScale();

	Window = SNew(SWindow)
	.Title( LOCTEXT("Create Chooser Options", "Create Chooser Table") )
	.SizingRule(ESizingRule::FixedSize)
	.ClientSize(FVector2f(AppScale*550, AppScale*300))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	[
		AsShared()
	];

	GEditor->EditorAddModalWindow(Window.ToSharedRef());
	ChooserFactory.Reset();

	return bOkClicked;
}


/** Handler for when ok is clicked */
FReply SChooserCreateDialog::OkClicked()
{
	CloseDialog(true);

	return FReply::Handled();
}

void SChooserCreateDialog::CloseDialog(bool bWasPicked)
{
	bOkClicked = bWasPicked;
	if (Window.IsValid())
	{
		Window->RequestDestroyWindow();
	}
}

/** Handler for when cancel is clicked */
FReply SChooserCreateDialog::CancelClicked()
{
	CloseDialog();
	return FReply::Handled();
}

FReply SChooserCreateDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		CloseDialog();
		return FReply::Handled();
	}
	return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

#undef LOCTEXT_NAMESPACE
