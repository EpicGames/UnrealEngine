// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "StructUtils/InstancedStruct.h"
#include "ChooserSignatureFactory.h"
#include "PropertyEditorModule.h"

/*------------------------------------------------------------------------------
	Dialog to configure creation properties
------------------------------------------------------------------------------*/
class SChooserCreateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SChooserCreateDialog ){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );
	
	/** Sets properties for the supplied AnimBlueprintFactory */
	bool ConfigureProperties(TWeakObjectPtr<UChooserSignatureFactory> InChooserFactory);

private:

	/** Handler for when ok is clicked */
	FReply OkClicked();
	
	void CloseDialog(bool bWasPicked=false);
	
	/** Handler for when cancel is clicked */
	FReply CancelClicked();
	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent );

private:
	/** The factory for which we are setting up properties */
	TWeakObjectPtr<UChooserSignatureFactory> ChooserFactory;
	TSharedPtr<SWindow> Window;
	TSharedPtr<IDetailsView> DetailsView;

	/** True if Ok was clicked */
	bool bOkClicked = false;
};