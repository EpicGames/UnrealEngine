// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserCreateRowButton.h"
#include "ChooserTableEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "SPositiveActionButton.h"
#include "UObject/UObjectIterator.h"

#include "ScopedTransaction.h"
#include "ObjectChooser_Class.h"

#define LOCTEXT_NAMESPACE "AddColumnButton"

namespace UE::ChooserEditor
{
	struct FResultTypeInfoStruct
	{
		bool ObjectOnly = false;
		bool ClassOnly = false;
		FString Category;
		UScriptStruct* Type = nullptr;
		
		bool operator < (const FResultTypeInfoStruct& Other) const
		{
			if (Category == Other.Category)
			{
				return Type->GetDisplayNameText().ToString() < Other.Type->GetDisplayNameText().ToString();
			}
			else
			{
				return Category < Other.Category;
			}
		}
	};

	DECLARE_DELEGATE_OneParam(FCreateStructDelegate, UScriptStruct*);

	void MakeCreateResultMenu(FMenuBuilder& MenuBuilder, EObjectChooserResultType ChooserResultType, FCreateStructDelegate CreateStruct)
	{
		static TArray<FResultTypeInfoStruct> ResultTypes;

		if (ResultTypes.IsEmpty())
		{
			UScriptStruct* BaseType = FObjectChooserBase::StaticStruct();
			for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
			{
				if (*StructIt != BaseType && StructIt->IsChildOf(BaseType))
				{
					if (!StructIt->HasMetaData("Hidden"))
					{
						FResultTypeInfoStruct Info;
						Info.Type = *StructIt;
						Info.Category = StructIt->HasMetaData("Category") ? StructIt->GetMetaData("Category") : "Other";

						if (StructIt->HasMetaData("ResultType"))
						{
							FString ResultTypeString = StructIt->GetMetaData("ResultType");
							Info.ClassOnly = ResultTypeString == "Class";
							Info.ObjectOnly = ResultTypeString == "Object";
						}
						
						ResultTypes.Add(Info);
					}
				}
			}
			ResultTypes.Sort();
		}

		FString Section = "";
		for(FResultTypeInfoStruct& Type : ResultTypes)
		{
			if (Section != Type.Category)
			{
				if (Section != "")
				{
					MenuBuilder.EndSection();
				}
				Section = Type.Category;
				MenuBuilder.BeginSection(FName(Section), FText::FromString(Section));
			}
			
			MenuBuilder.AddMenuEntry(Type.Type->GetDisplayNameText(), Type.Type->GetToolTipText(), FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([Type, CreateStruct]()
					{
						CreateStruct.Execute(Type.Type);
					}),
					FCanExecuteAction::CreateLambda([Type, ChooserResultType]()
					{
						if (Type.ClassOnly && ChooserResultType == EObjectChooserResultType::ObjectResult)
						{
							return false;
						}
						if (Type.ObjectOnly && ChooserResultType == EObjectChooserResultType::ClassResult)
						{
							return false;
						}
						return true;
					})
					)
				);
								
		}
	}

	

	TSharedRef<SWidget>	SChooserCreateRowButton::MakeCreateRowMenu()
    {
		FMenuBuilder MenuBuilder(true, nullptr);
	 
		UChooserTable* Chooser = ChooserViewModel->GetChooser();
		if (!Chooser->FallbackResult.IsValid())
		{
			if (Chooser->ResultType == EObjectChooserResultType::NoPrimaryResult)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Add Fallback Output", "Add Fallback Output"),
					LOCTEXT("Add Fallback Output Tooltip", "Add a Fallback row to the chooser, which will be used in the case where no other rows passed all filter columns"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this]()
					{
						UChooserTable* Chooser = ChooserViewModel->GetChooser();
						const FScopedTransaction Transaction(LOCTEXT("Add Fallback Row Transaction", "Add Fallback Row"));
						Chooser->Modify(true);
						
						// Just construct a dummy result to make sure all rows always have "valid results"
						// You can't just leave a null result otherwise rows don't apply their output.
						Chooser->FallbackResult.InitializeAs(FClassChooser::StaticStruct());
						Chooser->FallbackResult.GetMutable<FClassChooser>().Class = UClass::StaticClass();
						
						ChooserViewModel->UpdateTableRows();
					}))
				);
			}
			else
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("Add Fallback", "Add Fallback Result"),
						LOCTEXT("Add Fallback Tooltip", "Add a Fallback row to the chooser, which will be used in the case where no other rows passed all filter columns"),
						FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
						{
							UChooserTable* Chooser = ChooserViewModel->GetChooser();
							MakeCreateResultMenu(MenuBuilder, Chooser->GetContextOwner()->ResultType, FCreateStructDelegate::CreateLambda([this](UScriptStruct* Type)
							{
								UChooserTable* Chooser = ChooserViewModel->GetChooser();
								const FScopedTransaction Transaction(LOCTEXT("Add Fallback Row Transaction", "Add Fallback Row"));
								Chooser->Modify(true);
								
								Chooser->FallbackResult.InitializeAs(Type);
								
								ChooserViewModel->UpdateTableRows();
							}));
						})
					);
			}
		}
		
		if (Chooser->ResultType == EObjectChooserResultType::NoPrimaryResult)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Add Output Row", "Add Output Row"),
				LOCTEXT("Add Output Row Tooltip", "Add a regular row to the chooser"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]()
				{
					UChooserTable* Chooser = ChooserViewModel->GetChooser();
					const FScopedTransaction Transaction(LOCTEXT("Add Row Transaction", "Add Row"));
					Chooser->Modify(true);
	 
					// Just construct a dummy result to make sure all rows always have "valid results"
					// You can't just leave a null result otherwise rows don't apply their output.
					FInstancedStruct& NewResult = Chooser->ResultsStructs.AddDefaulted_GetRef();
					NewResult.InitializeAs(FClassChooser::StaticStruct());
					NewResult.GetMutable<FClassChooser>().Class = UClass::StaticClass();
					
					ChooserViewModel->UpdateTableRows();
				}))
			);
		}
		else
		{
			MakeCreateResultMenu(MenuBuilder, Chooser->GetContextOwner()->ResultType, FCreateStructDelegate::CreateLambda([this](UScriptStruct* Type)
			{
				UChooserTable* Chooser = ChooserViewModel->GetChooser();
				const FScopedTransaction Transaction(LOCTEXT("Add Row Transaction", "Add Row"));
				Chooser->Modify(true);
				
				FInstancedStruct& NewResult = Chooser->ResultsStructs.AddDefaulted_GetRef();
				NewResult.InitializeAs(Type);
				
				ChooserViewModel->UpdateTableRows();
			}));
		}
	 
		return MenuBuilder.MakeWidget();
    }
		
	void SChooserCreateRowButton::Construct(const FArguments& InArgs)
	{
		ChooserViewModel = InArgs._ViewModel;
		check(ChooserViewModel);
		
		ChildSlot
		[
			SNew(SPositiveActionButton)
				.Text(LOCTEXT("Add Row", "Add Row"))
            	.OnGetMenuContent(this, &SChooserCreateRowButton::MakeCreateRowMenu)
		];

	}

	SChooserCreateRowButton::~SChooserCreateRowButton()
	{
	}

}

#undef LOCTEXT_NAMESPACE
