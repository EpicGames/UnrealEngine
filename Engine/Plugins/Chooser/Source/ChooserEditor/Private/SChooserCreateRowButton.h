// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Commands/UICommandList.h"
#include "SChooserTableRow.h"
#include "ChooserTableEditor.h"

class SPositiveActionButton;

namespace UE::ChooserEditor
{
	
class SChooserCreateRowButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChooserCreateRowButton)
	{}
	SLATE_ARGUMENT(TSharedPtr<FChooserTableViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SChooserCreateRowButton() override;

 private:
	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	TSharedRef<SWidget>	MakeCreateRowMenu();
};

}
