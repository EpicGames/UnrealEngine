// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserRowHandle.h"
#include "ChooserTableEditor.h"
#include "Misc/ITransaction.h"

#define LOCTEXT_NAMESPACE "ChooserRowHandle"

namespace UE::ChooserEditor
{

	TSharedRef<FChooserRowDragDropOp> FChooserRowDragDropOp::New(TSharedPtr<FChooserTableViewModel> InViewModel, uint32 InRowIndex)
	{
		TSharedRef<FChooserRowDragDropOp> Operation = MakeShareable(new FChooserRowDragDropOp());
		Operation->DefaultHoverText = LOCTEXT("Chooser Row", "Chooser Row");
		Operation->CurrentHoverText = Operation->DefaultHoverText;

		Operation->TransactionIndex = GEditor->BeginTransaction(LOCTEXT("Drag Chooser Table Rows", "Drag Chooser Table Rows"));

		Operation->RowData = InViewModel->CopyRowsInternal(InViewModel->GetSelectedRows());
		InViewModel->DeleteRowsInternal(InViewModel->GetSelectedRows());
		Operation->ChooserViewModel = InViewModel;
		
		Operation->Construct();

		return Operation;
	}

	void FChooserRowDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
	{
		if (!bDropWasHandled)
		{
			// we need to call Apply on the global undo, or cancelling the transaction doesn't actually roll back
			GUndo->Apply();
			GEditor->CancelTransaction(TransactionIndex);

			// PostUndo doesn't get called for cancelled transactions, so refresh the editor manually
			ChooserViewModel->RefreshAll();
		}
		FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	}

	void SChooserRowHandle::Construct(const FArguments& InArgs, bool bShowImage)
	{
		ChooserViewModel = InArgs._ViewModel;
		RowIndex = InArgs._RowIndex;

		if (bShowImage)
		{
			ChildSlot
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SBox) .Padding(0.0f) .HAlign(HAlign_Center) .VAlign(VAlign_Center) .WidthOverride(16.0f)
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SBox).Padding(0.0f) .HAlign(HAlign_Center) .VAlign(VAlign_Center) .WidthOverride(16.0f)
					[
						SNew(SImage)
						.Visibility_Lambda([this]()
						{
							return ChooserViewModel->GetChooser()->GetDebugTestValuesValid() && ChooserViewModel->GetChooser()->GetDebugSelectedRows().Contains(RowIndex)  ? EVisibility::HitTestInvisible : EVisibility::Hidden;
						})
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
					]
				]
			];
		}
	}
	
	FReply SChooserRowHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		// act as a move handle if the row is already selected, and if there are multiselect modifiers pressed
		if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton && !MouseEvent.IsControlDown() && !MouseEvent.IsShiftDown()
			&& ChooserViewModel->IsRowSelected(RowIndex))
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}
		else
		{
			return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
		}
	};

	FReply SChooserRowHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		TSharedRef<FChooserRowDragDropOp> DragDropOp = FChooserRowDragDropOp::New(ChooserViewModel, RowIndex);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

}

#undef LOCTEXT_NAMESPACE
