// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class UChooserTable;

namespace UE::ChooserEditor
{

class FChooserTableViewModel;

class FChooserRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FChooserRowDragDropOp, FDecoratedDragDropOp)

	UChooserTable* RowData = nullptr;
	int32 TransactionIndex = -1;
	TSharedPtr<FChooserTableViewModel> ChooserViewModel;

	/** Constructs the drag drop operation */
	static TSharedRef<FChooserRowDragDropOp> New(TSharedPtr<FChooserTableViewModel> InViewModel, uint32 InRowIndex);

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
};

class SChooserRowHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChooserRowHandle)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_ARGUMENT(TSharedPtr<FChooserTableViewModel>, ViewModel)
	SLATE_ARGUMENT(uint32, RowIndex)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, bool bShowImage);
	
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	uint32 RowIndex = 0;
};

}
