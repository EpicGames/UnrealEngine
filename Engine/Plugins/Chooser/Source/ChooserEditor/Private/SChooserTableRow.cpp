// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserTableRow.h"
#include "Chooser.h"
#include "ChooserEditorStyle.h"
#include "ChooserTableEditor.h"
#include "IContentBrowserSingleton.h"
#include "IObjectChooser.h"
#include "ObjectChooserWidgetFactories.h"
#include "ObjectChooser_Asset.h"
#include "SChooserRowHandle.h"
#include "ScopedTransaction.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SSeparator.h"
#include "Chooser.h"

#include "SChooserCreateRowButton.h"

#define LOCTEXT_NAMESPACE "ChooserTableRow"

namespace UE::ChooserEditor
{
	static const FLinearColor DisabledColor(0.0105f, 0.0105f, 0.0105f, 0.5f);
	static const FLinearColor TestPassedColor(0.0f, 1.0f, 0.0f, 0.3f);
	static const FLinearColor TestFailedColor(1.0f, 0.0f, 0.0f, 0.2f);
	
	void SChooserTableRow::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		TableHasFocus = Args._TableHasFocus;
		RowIndex = Args._Entry;
		Chooser = Args._Chooser;
		ChooserViewModel = Args._ViewModel;
		check(ChooserViewModel);

		SMultiColumnTableRow<TSharedPtr<FChooserTableRow>>::Construct(
			FSuperRowType::FArguments(),
			OwnerTableView
		);

		if (RowIndex->RowIndex >=0)
		{
			SetContent(SNew(SOverlay)
					+ SOverlay::Slot()
					[
						Content.Pin().ToSharedRef()
					]
					+ SOverlay::Slot().VAlign(VAlign_Bottom)
					[
						SNew(SSeparator)
						.SeparatorImage(FAppStyle::GetBrush("PropertyEditor.HorizontalDottedLine"))
						.ColorAndOpacity_Lambda([this]() { return FSlateColor(bDropSupported ? EStyleColor::Select : EStyleColor::Error); })
						.Visibility_Lambda([this]() { return bDragActive && !bDropAbove ? EVisibility::Visible : EVisibility::Hidden; })
					]
					+ SOverlay::Slot().VAlign(VAlign_Top)
					[
						SNew(SSeparator)
						.SeparatorImage(FAppStyle::GetBrush("PropertyEditor.HorizontalDottedLine"))
						.ColorAndOpacity_Lambda([this]() { return FSlateColor(bDropSupported ? EStyleColor::Select : EStyleColor::Error); }) 
						.Visibility_Lambda([this]() { return bDragActive && bDropAbove ? EVisibility::Visible : EVisibility::Hidden; })
					]
				);
		}
		else if (RowIndex->RowIndex == SpecialIndex_Fallback || RowIndex->RowIndex == SpecialIndex_AddRow )
		{
			SetContent(
					SNew(SOverlay)
							+ SOverlay::Slot()
							[
								Content.Pin().ToSharedRef()
							]
							+ SOverlay::Slot().VAlign(VAlign_Top)
							[
								SNew(SSeparator)
								.SeparatorImage(FAppStyle::GetBrush("PropertyEditor.HorizontalDottedLine"))
								.ColorAndOpacity_Lambda([this]() { return FSlateColor(bDropSupported ? EStyleColor::Select : EStyleColor::Error); }) 
								.Visibility_Lambda([this]() { return bDragActive ? EVisibility::Visible : EVisibility::Hidden; })
							]
				);
		}
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	TSharedRef<SWidget> SChooserTableRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		static FName Result = "Result";
		static FName Handles = "Handles";
		static FName AddColumn = "Add";
	
		if (Chooser->ResultsStructs.IsValidIndex(RowIndex->RowIndex))
		{
			if (ColumnName == AddColumn || ColumnName == Handles)
			{
				bool bShowHandleImage = ColumnName == Handles;
				
				// row drag handle
				return SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SChooserRowHandle, bShowHandleImage).ViewModel(ChooserViewModel).RowIndex(RowIndex->RowIndex)
						]
						+ SOverlay::Slot()
						[
							SNew(SColorBlock).Color(DisabledColor)
									.Visibility_Lambda(
									[this]()
									{
										if (Chooser->IsRowDisabled(RowIndex->RowIndex))
										{
											return EVisibility::HitTestInvisible;
										}
										return EVisibility::Hidden;
									})
									
						];
			}
			else if (ColumnName == Result) 
			{
				FChooserWidgetValueChanged ChooserWidgetValueChanged = FChooserWidgetValueChanged::CreateLambda([this]()
					{
						check(ChooserViewModel);
						ChooserViewModel->AutoPopulateRow(RowIndex->RowIndex);
					});

				UChooserTable* ContextOwner = Chooser->GetRootChooser();
				TSharedPtr<SWidget> ResultWidget = FObjectChooserWidgetFactories::CreateWidget(
					false, 
					Chooser,
					FObjectChooserBase::StaticStruct(),
					Chooser->ResultsStructs[RowIndex->RowIndex].GetMutableMemory(),
					Chooser->ResultsStructs[RowIndex->RowIndex].GetScriptStruct(),
					ContextOwner->OutputObjectType,

					FOnStructPicked::CreateLambda([this, InRowIndex = RowIndex->RowIndex, ChooserWidgetValueChanged](const UScriptStruct* ChosenStruct)
				{
					UChooserTable* ContextOwner = Chooser->GetRootChooser();
					const FScopedTransaction Transaction(LOCTEXT("Change Row Result Type", "Change Row Result Type"));
					Chooser->Modify(true);
					Chooser->ResultsStructs[InRowIndex].InitializeAs(ChosenStruct);
							FObjectChooserWidgetFactories::CreateWidget(
								false,
								Chooser,
								FObjectChooserBase::StaticStruct(),
								Chooser->ResultsStructs[InRowIndex].GetMutableMemory(),
								ChosenStruct,
								ContextOwner->OutputObjectType,
								FOnStructPicked(),
								&CacheBorder,
								ChooserWidgetValueChanged);
				}),
					&CacheBorder,
					ChooserWidgetValueChanged,
					ChooserViewModel.Get()
				);

				return SNew(SOverlay)
						+ SOverlay::Slot()
						[
							ResultWidget.ToSharedRef()
						]
						+ SOverlay::Slot()
						[
							SNew(SColorBlock).Color(DisabledColor)
									.Visibility_Lambda(
									[this]()
									{
										if (Chooser->IsRowDisabled(RowIndex->RowIndex))
										{
											return EVisibility::HitTestInvisible;
										}
										return EVisibility::Hidden;
									})
									
						];
			}
			else
			{
				const int ColumnIndex = ColumnName.GetNumber() - 1;
				if (ColumnIndex < Chooser->ColumnsStructs.Num() && ColumnIndex >=0)
				{
					FChooserColumnBase* Column = &Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
					const UStruct * ColumnStruct = Chooser->ColumnsStructs[ColumnIndex].GetScriptStruct();

					TSharedPtr<SWidget> ColumnWidget = FObjectChooserWidgetFactories::CreateColumnWidget(Column, ColumnStruct, Chooser, RowIndex->RowIndex);
				
					if (ColumnWidget.IsValid())
					{
						return SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBorder)
							.BorderBackgroundColor(FLinearColor(0,0,0,0))
							.Padding(FMargin(4,0))
							.Content()
							[
								SNew(SOverlay)
								+ SOverlay::Slot()
								[
									SNew(SColorBlock).Color_Lambda([this, ColumnIndex](){ return TableHasFocus.Get() ? FStyleColors::Select.GetSpecifiedColor() : FStyleColors::SelectInactive.GetSpecifiedColor(); } )
										.Visibility_Lambda(
										[this, ColumnIndex]()
										{
											if (ChooserViewModel->IsColumnSelected(ColumnIndex))
											{
												return EVisibility::Visible;
											}
											return EVisibility::Hidden;
										})
								]
								+ SOverlay::Slot()
								[
									ColumnWidget.ToSharedRef()
								]
							]
						]
						+ SOverlay::Slot()
						[
							SNew(SColorBlock).Visibility(EVisibility::HitTestInvisible).Color_Lambda(
									[this,Column]()
									{
										if (Chooser->GetDebugTestValuesValid())
										{
											if (Column->HasCosts())
											{
												if (!Column->HasFilters() || Column->EditorTestFilter(RowIndex->RowIndex))
												{
													const double Cost = Column->EditorTestCost(RowIndex->RowIndex);

													const FLinearColor Low(0.0f, 1.0f, 0.0f, 0.30f);
													const FLinearColor Mid(0.8f, 0.8f, 0.0f, 0.30f);
													const FLinearColor High(1.0f, 0.0f, 0.0f, 0.20f);

													if (Cost < 0.5)
													{
														return FLinearColor::LerpUsingHSV(Low, Mid, FMath::Clamp(Cost * 2.0, 0.0f, 1.0f));
													}
													else
													{
														return FLinearColor::LerpUsingHSV(Mid, High, FMath::Clamp((Cost - 0.5) * 2.0, 0.0f, 1.0f));
													}
												}
											}
											else if (Column->HasFilters())
											{
												if (Column->EditorTestFilter(RowIndex->RowIndex))
												{
													return TestPassedColor;
												}
												else
												{
													return TestFailedColor;
												}
											}
										}
										return FLinearColor::Transparent;
									})
									
						]
						+ SOverlay::Slot()
						[
							SNew(SColorBlock).Color(DisabledColor)
									.Visibility_Lambda(
									[this,Column]()
									{
										if (Chooser->IsRowDisabled(RowIndex->RowIndex) || Column->bDisabled)
										{
											return EVisibility::HitTestInvisible;
										}
										return EVisibility::Hidden;
									})
									
						]
						;
					}
				}
			}
		}
		else if (RowIndex->RowIndex == SpecialIndex_Fallback)
		{
			if (ColumnName == Handles)
			{
				return SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SBox).Padding(0.0f) .HAlign(HAlign_Center) .VAlign(VAlign_Center) .WidthOverride(16.0f)
                    					[
                    						SNew(SImage)
                    						.Image(FChooserEditorStyle::Get().GetBrush("ChooserEditor.FallbackIcon"))
                    						.ToolTipText(LOCTEXT("FallbackTooltip","Fallback result:  Returned if all rows failed."))
                    					]
				]
				+ SOverlay::Slot()
				[
					SNew(SBox).Padding(0.0f) .HAlign(HAlign_Center) .VAlign(VAlign_Center) .WidthOverride(16.0f)
					[
						SNew(SImage)
						.Visibility_Lambda([this]()
						{
							return Chooser->GetDebugTestValuesValid() && Chooser->GetDebugSelectedRows().Contains(RowIndex->RowIndex) ? EVisibility::HitTestInvisible : EVisibility::Hidden;
						})
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
					]
				];
			}
			else if (ColumnName == Result) 
			{
				FChooserWidgetValueChanged ChooserWidgetValueChanged = FChooserWidgetValueChanged::CreateLambda([this]()
					{
						check(ChooserViewModel);
						ChooserViewModel->AutoPopulateRow(RowIndex->RowIndex);
					});

				UChooserTable* ContextOwner = Chooser->GetRootChooser();
				TSharedPtr<SWidget> ResultWidget = FObjectChooserWidgetFactories::CreateWidget(
					false,
					Chooser,
					FObjectChooserBase::StaticStruct(),
					Chooser->FallbackResult.GetMutableMemory(),
					Chooser->FallbackResult.GetScriptStruct(), ContextOwner->OutputObjectType,
					FOnStructPicked::CreateLambda([this, InRowIndex = RowIndex->RowIndex, ChooserWidgetValueChanged](const UScriptStruct* ChosenStruct)
						{
							UChooserTable* ContextOwner = Chooser->GetRootChooser();
							const FScopedTransaction Transaction(LOCTEXT("Change Row Result Type", "Change Row Result Type"));
							Chooser->Modify(true);
							Chooser->FallbackResult.InitializeAs(ChosenStruct);
							FObjectChooserWidgetFactories::CreateWidget(
								false,
								Chooser,
								FObjectChooserBase::StaticStruct(),
								Chooser->FallbackResult.GetMutableMemory(),
								ChosenStruct,
								ContextOwner->OutputObjectType,
								FOnStructPicked(),
								&CacheBorder,
								ChooserWidgetValueChanged,
								ChooserViewModel.Get(),
								LOCTEXT("Fallback Result", "Fallback Result: (None)")
							);
						}),
					&CacheBorder,
					ChooserWidgetValueChanged,
					ChooserViewModel.Get(),
					LOCTEXT("Fallback Result", "Fallback Result: (None)")
				);
				
				return ResultWidget.ToSharedRef();
			}
			else
			{
				const int ColumnIndex = ColumnName.GetNumber() - 1;
				if (ColumnIndex < Chooser->ColumnsStructs.Num() && ColumnIndex >=0)
				{
					FChooserColumnBase* Column = &Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();
					const UStruct * ColumnStruct = Chooser->ColumnsStructs[ColumnIndex].GetScriptStruct();

					TSharedPtr<SWidget> ColumnWidget = FObjectChooserWidgetFactories::CreateColumnWidget(Column, ColumnStruct, Chooser->GetRootChooser(), -2);
				
					if (ColumnWidget.IsValid())
					{
						return SNew(SOverlay)
							+ SOverlay::Slot()
							[
								SNew(SBorder)
								.BorderBackgroundColor(FLinearColor(0,0,0,0))
								.Padding(FMargin(4,0))
								.Content()
								[
									SNew(SOverlay)
									+SOverlay::Slot()
									[
										SNew(SColorBlock).Color_Lambda([this, ColumnIndex](){ return TableHasFocus.Get() ? FStyleColors::Select.GetSpecifiedColor() : FStyleColors::SelectInactive.GetSpecifiedColor(); } )
											.Visibility_Lambda(
											[this, ColumnIndex]()
											{
												if (ChooserViewModel->IsColumnSelected(ColumnIndex))
												{
													return EVisibility::Visible;
												}
												return EVisibility::Hidden;
											})
									]
									+ SOverlay::Slot()
									[
										ColumnWidget.ToSharedRef()
									]
								]
							];
					}
				}
			}
		}
		else if (RowIndex->RowIndex == SpecialIndex_AddRow)
		{
			// on the row past the end, show an Add button in the first column available
			bool bIsLeftmostColumn;
			if (Chooser->ResultType != EObjectChooserResultType::NoPrimaryResult)
			{
				bIsLeftmostColumn = ColumnName == Result;
			}
			else if (Chooser->ColumnsStructs.IsEmpty())
			{
				bIsLeftmostColumn = ColumnName == AddColumn;
			}
			else
			{
				bIsLeftmostColumn = ColumnName.GetNumber() == 1;
			}
			
			if (bIsLeftmostColumn)
			{
				return SNew(SHorizontalBox)
								+SHorizontalBox::Slot().AutoWidth()
								[
									SNew(SChooserCreateRowButton).ViewModel(ChooserViewModel)
								];
			}
		}

		return SNullWidget::NullWidget;
	}

	void SChooserTableRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		bDropSupported = false;
		if (TSharedPtr<FChooserRowDragDropOp> Operation = DragDropEvent.GetOperationAs<FChooserRowDragDropOp>())
		{
			bDropSupported = true;
		}
		else if (TSharedPtr<FAssetDragDropOp> ContentDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{
			if (Chooser->ResultType == EObjectChooserResultType::ObjectResult)
			{
				bDropSupported = true;
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				UObject* Object = static_cast<UObject*>(Chooser);
				AssetReferenceFilterContext.AddReferencingAssets(TArrayView(&Object, 1));
				TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;	
			
				UChooserTable* ContextOwner = Chooser->GetRootChooser();
				for (const FAssetData& Asset : ContentDragDropOp->GetAssets())
				{
					if (AssetReferenceFilter.IsValid())
					{
						if (!AssetReferenceFilter->PassesFilter(Asset))
						{
							bDropSupported = false;
							break;
						}
					}
					
					if (ContextOwner->OutputObjectType) // if OutputObjectType is null, then any kind of object is supported, don't need to check all of them
					{
						const UClass* AssetClass = Asset.GetClass();

						if(AssetClass->IsChildOf(UChooserTable::StaticClass()))
						{
							const UChooserTable* DraggedChooserTable = Cast<UChooserTable>(Asset.GetAsset());

							// verify dragged chooser result type matches this chooser result type
							if (DraggedChooserTable->ResultType == EObjectChooserResultType::ClassResult
							    || DraggedChooserTable->OutputObjectType == nullptr
							    || !DraggedChooserTable->OutputObjectType->IsChildOf(ContextOwner->OutputObjectType))
							{
								bDropSupported = false;
								break;
							}
						}
						else if(!AssetClass->IsChildOf(ContextOwner->OutputObjectType))
                        {
							bDropSupported = false;
							break;
                        }
					}
				}
			}
		}
		
		float Center = MyGeometry.Position.Y + MyGeometry.Size.Y;
		bDropAbove = DragDropEvent.GetScreenSpacePosition().Y < Center;
		bDragActive = true;
	}
	void SChooserTableRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		bDragActive = false;
	}

	FReply SChooserTableRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		DragActiveCounter = 2;
		bDragActive = true;
		float Center = MyGeometry.AbsolutePosition.Y + MyGeometry.Size.Y/2;
		bDropAbove = DragDropEvent.GetScreenSpacePosition().Y < Center;
		return FReply::Handled();
	}

	FReply SChooserTableRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		bDragActive = false;
		
		if (!bDropSupported)
		{
			return FReply::Unhandled();
		}
		
		if (TSharedPtr<FChooserRowDragDropOp> Operation = DragDropEvent.GetOperationAs<FChooserRowDragDropOp>())
		{
			int NewRowIndex;
			if (!Chooser->ResultsStructs.IsValidIndex(RowIndex->RowIndex))
			{
				// for special (negative) indices, move to the end
				NewRowIndex = Chooser->ResultsStructs.Num();
			}
			else if (bDropAbove)
			{
				NewRowIndex = RowIndex->RowIndex;
			}
			else
			{
				NewRowIndex = RowIndex->RowIndex+1;
			}
			ChooserViewModel->PasteInternal(Operation->RowData, NewRowIndex);
			GEditor->EndTransaction();
		}
		else if (TSharedPtr<FAssetDragDropOp> ContentDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("DragDropAssets","Drag and Drop Assets into Chooser"));
			Chooser->Modify(true);
			
			if (Chooser->ResultType == EObjectChooserResultType::ObjectResult)
			{
				int InsertRowIndex = RowIndex->RowIndex;
				if (!Chooser->ResultsStructs.IsValidIndex(RowIndex->RowIndex))
				{
					// for special (negative) indices, move to the end
					InsertRowIndex = Chooser->ResultsStructs.Num();
				}
				else if (!bDropAbove)
				{
					InsertRowIndex = RowIndex->RowIndex + 1;
				}
				
				if (bDropSupported)
				{
					TArray<FInstancedStruct> NewResults;
					const TArray<FAssetData>& AssetList = ContentDragDropOp->GetAssets();
					NewResults.Reserve(AssetList.Num());
					
					for (const FAssetData& Asset : AssetList)
					{
						const UClass* AssetClass = Asset.GetClass();
						
						NewResults.SetNum(NewResults.Num()+1);
						FInstancedStruct& NewResult = NewResults.Last();

						if(AssetClass->IsChildOf(UChooserTable::StaticClass()))
						{
							NewResult.InitializeAs(FEvaluateChooser::StaticStruct());
							NewResult.GetMutable<FEvaluateChooser>().Chooser = Cast<UChooserTable>(Asset.GetAsset());
						}
						else
						{
							UChooserTable* ContextOwner = Chooser->GetRootChooser();
							if (ContextOwner->OutputObjectType == nullptr || ensure(AssetClass->IsChildOf(ContextOwner->OutputObjectType)))
							{
								NewResult.InitializeAs(FAssetChooser::StaticStruct());
								NewResult.GetMutable<FAssetChooser>().Asset = Asset.GetAsset();
							}
						}
					}

					Chooser->ResultsStructs.Insert(NewResults, InsertRowIndex);
					Chooser->DisabledRows.InsertZeroed(InsertRowIndex, NewResults.Num());
					
					// Make sure each column has the same number of row datas as there are results
					for(FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
					{
						FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
						Column.InsertRows(InsertRowIndex, NewResults.Num());
					}

					ChooserViewModel->ClearSelectedRows();
					ChooserViewModel->RefreshAll();
					
					TArray<int32, TInlineAllocator<256>> RowsToSelect;
					RowsToSelect.Reserve(NewResults.Num());
					for(int Index = InsertRowIndex; Index < InsertRowIndex + NewResults.Num(); Index++)
					{
						ChooserViewModel->AutoPopulateRow(Index);
						RowsToSelect.Add(Index);
					}
					ChooserViewModel->SelectRows(RowsToSelect);
				}
			}
		}
				
		return FReply::Handled();		
	}

	
	void SChooserTableRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		DragActiveCounter --;
		if (DragActiveCounter < 0)
		{
			DragActiveCounter = 0;
			bDragActive = false;
		}
		
		SMultiColumnTableRow<TSharedPtr<FChooserTableRow, ESPMode::ThreadSafe>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

#undef LOCTEXT_NAMESPACE
