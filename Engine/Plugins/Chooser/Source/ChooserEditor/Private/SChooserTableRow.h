// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class UChooserTable;

namespace UE::ChooserEditor
{

class FChooserTableViewModel;

struct FChooserTableRow
{
	FChooserTableRow(int32 i) { RowIndex = i; }
	int32 RowIndex;
};

class SChooserTableRow : public SMultiColumnTableRow<TSharedPtr<FChooserTableRow>>
{
public:
SLATE_BEGIN_ARGS(SChooserTableRow) {}
/** The list item for this row */
	SLATE_ARGUMENT(TSharedPtr<FChooserTableRow>, Entry)
	SLATE_ARGUMENT(UChooserTable*, Chooser)
	SLATE_ARGUMENT(TSharedPtr<FChooserTableViewModel>, ViewModel)
	SLATE_ATTRIBUTE(bool, TableHasFocus);
	SLATE_END_ARGS()

	enum { SpecialIndex_AddRow = -1, SpecialIndex_Fallback = -2 };

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TAttribute<bool> TableHasFocus;
	TSharedPtr<FChooserTableRow> RowIndex;
	UChooserTable* Chooser;
	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	TSharedPtr<SBorder> CacheBorder;
	int DragActiveCounter = 0;
	bool bDragActive = false;
	bool bDropSupported = false;
	bool bDropAbove = false;
};

}
