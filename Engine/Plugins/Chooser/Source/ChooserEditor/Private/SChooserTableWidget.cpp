// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChooserTableWidget.h"
#include "ChooserTableEditor.h"
#include "Chooser.h"
#include "Framework/Application/SlateApplication.h"
#include "ObjectTools.h"
#include "OutputObjectColumn.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "ObjectChooserWidgetFactories.h"
#include "SChooserCreateColumnButton.h"
#include "SChooserColumnHandle.h"
#include "SChooserTableRow.h"
#include "ToolMenus.h"
#include "ChooserTableEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"


#define LOCTEXT_NAMESPACE "ChooserTableWidget"

namespace UE::ChooserEditor
{
	
	bool SChooserTableWidget::TableHasFocus() const
	{
		if (TableView)
		{
			return TableView->HasKeyboardFocus();
		}
		return false;
	}
	
	void SChooserTableWidget::ClearSelection()
	{
		if (TableView)
		{
			TableView->ClearSelection();
		}
	}
	
	void SChooserTableWidget::GetSelectedItems(TArray<TSharedPtr<FChooserTableRow>>& SelectedItems)
	{
		TableView->GetSelectedItems(SelectedItems);
	}

	int SChooserTableWidget::GetSelectedColumn() const
	{
		if (ChooserViewModel)
		{
			return ChooserViewModel->GetSelectedColumn();
		}
		return INDEX_NONE;
	}
	
	void SChooserTableWidget::UpdateTableColumns()
	{
		if (ChooserViewModel)
		{
			UChooserTable* Chooser = ChooserViewModel->GetChooser();

			HeaderRow->ClearColumns();

			HeaderRow->AddColumn(SHeaderRow::Column("Handles")
							.DefaultLabel(FText())
							.ManualWidth(30));

			if (Chooser->ResultType != EObjectChooserResultType::NoPrimaryResult)
			{
				HeaderRow->AddColumn(SHeaderRow::Column("Result")
								.ManualWidth_Lambda([Chooser]() { return Chooser->EditorResultsColumnWidth; } )
								.OnWidthChanged_Lambda([Chooser](float NewWidth) { Chooser->EditorResultsColumnWidth = NewWidth; })
								.HeaderContent()
								[
									SNew(SVerticalBox)
									+ SVerticalBox::Slot()
									.VAlign(VAlign_Top)
									[
										SNew(STextBlock)
											.Text(LOCTEXT("Result", "Result"))
											.ToolTipText(LOCTEXT("ResultTooltip", "The Result is the asset which will be returned if a row is selected (or other Chooser to evaluate to get the asset to return"))
									]
								]);
			}

			FName ColumnId("ChooserColumn", 1);
			int NumColumns = Chooser->ColumnsStructs.Num();	
			for(int ColumnIndex = 0; ColumnIndex < NumColumns; ColumnIndex++)
			{
				FChooserColumnBase& Column = Chooser->ColumnsStructs[ColumnIndex].GetMutable<FChooserColumnBase>();

				TSharedPtr<SWidget> HeaderWidget = FObjectChooserWidgetFactories::CreateColumnWidget(&Column, Chooser->ColumnsStructs[ColumnIndex].GetScriptStruct(), Chooser->GetRootChooser(), -1);
				if (!HeaderWidget.IsValid())
				{
					HeaderWidget = SNullWidget::NullWidget;
				}
				
				HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments()
					.ColumnId(ColumnId)
					.ManualWidth(Column.EditorColumnWidth)
					.ManualWidth_Lambda([&Column]() { return Column.EditorColumnWidth; } )
					.OnWidthChanged_Lambda([&Column](float NewWidth) { Column.EditorColumnWidth = NewWidth; })
					.HeaderComboVisibility(EHeaderComboVisibility::Ghosted)
					.HeaderContent()
					[
						SNew(SChooserColumnHandle)
							.ViewModel(ChooserViewModel)
							.TableHasFocus_Raw(this, &SChooserTableWidget::TableHasFocus)
							.ColumnIndex(ColumnIndex)
							.NoDropAfter(Chooser->ColumnsStructs[ColumnIndex].Get<FChooserColumnBase>().IsRandomizeColumn())
						[
							HeaderWidget.ToSharedRef()
						]
					
					]);
			
				ColumnId.SetNumber(ColumnId.GetNumber() + 1);
			}

			HeaderRow->AddColumn( SHeaderRow::FColumn::FArguments()
				.ColumnId("Add")
				.FillWidth(1.0)
				.HeaderContent( )
				[
					SNew(SVerticalBox)
					 + SVerticalBox::Slot().AutoHeight()
					 [
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot().MaxWidth(150)
						[
							CreateColumnComboButton.ToSharedRef()
						]
					]
				]
				);
		}
	}

	void SChooserTableWidget::UpdateTableRows()
	{
		UChooserTable* Chooser = ChooserViewModel->GetChooser();

		int32 NewNum = Chooser->ResultsStructs.Num();
		Chooser->DisabledRows.SetNum(NewNum);
    
		// Sync the TableRows array which drives the ui table to match the number of results.
		TableRows.SetNum(0, EAllowShrinking::No);
		for(int i =0; i < NewNum; i++)
		{
			TableRows.Add(MakeShared<FChooserTableRow>(i));
		}
    
		// Add one at the end, for the Fallback result
		if (Chooser->FallbackResult.IsValid())
		{
			TableRows.Add(MakeShared<FChooserTableRow>(SChooserTableRow::SpecialIndex_Fallback));
		}
    	
		// Add one at the end, for the "Add Row" control
		TableRows.Add(MakeShared<FChooserTableRow>(SChooserTableRow::SpecialIndex_AddRow));
    
		// Make sure each column has the same number of row datas as there are results
		for(FInstancedStruct& ColumnData : Chooser->ColumnsStructs)
		{
			FChooserColumnBase& Column = ColumnData.GetMutable<FChooserColumnBase>();
			Column.SetNumRows(NewNum);
		}
    
		if (TableView.IsValid())
		{
			TableView->RebuildList();
		}
	}
			
	TSharedPtr<SWidget> SChooserTableWidget::GenerateRowContextMenu()
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		FToolMenuContext ToolMenuContext;
		ToolMenuContext.AppendCommandList(CommandList);
		return ToolMenus->GenerateWidget(FChooserTableEditor::ContextMenuName, ToolMenuContext);
	}
	
	TSharedRef<ITableRow> SChooserTableWidget::GenerateTableRow(TSharedPtr<FChooserTableRow> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		UChooserTable* Chooser = ChooserViewModel->GetChooser();

		return SNew(SChooserTableRow, OwnerTable)
			.Entry(InItem).Chooser(Chooser).ViewModel(ChooserViewModel).TableHasFocus(this, &SChooserTableWidget::TableHasFocus);
	}

	void SChooserTableWidget::SelectRow(int32 RowIndex, bool bClear)
    {
    	if (TSharedPtr<FChooserTableRow>* Row = TableRows.FindByPredicate([RowIndex](const TSharedPtr<FChooserTableRow>& InRow)
    		{
    			return InRow->RowIndex == RowIndex;
    		}))
    	{
    		if (!TableView->IsItemSelected(*Row))
    		{
    			if (bClear)
    			{
    				TableView->ClearSelection();
    			}
    			TableView->SetItemSelection(*Row, true, ESelectInfo::OnMouseClick);
    		}
    	}
    }
    	
    void SChooserTableWidget::SelectRows(const TConstArrayView<int32> Rows)
    {
    	
    	TableView->ClearSelection();
    	TArray<TSharedPtr<FChooserTableRow>> RowsToSelect;
    
    	for (TSharedPtr<FChooserTableRow>& Row : TableRows)
    	{
    		if (Rows.Contains(Row->RowIndex))
    		{
    			RowsToSelect.Add(Row);
    		}
    	}
    	
    	TableView->SetItemSelection(RowsToSelect, true);
    }
	
	
	void SChooserTableWidget::Construct(const FArguments& InArgs)
	{
		ChooserViewModel = StaticCastSharedPtr<FChooserTableViewModel>(InArgs._ViewModel);
		check(ChooserViewModel);

		BindCommands(InArgs._Commands);

		ChooserViewModel->RefreshTableRowsDelegate = FRefreshTable::CreateLambda([WeakPtr = AsWeak()]()
		{
			if (TSharedPtr<SWidget> Ptr = WeakPtr.Pin())
			{
				StaticCastSharedPtr<SChooserTableWidget>(Ptr)->UpdateTableRows();
			}
		});

		ChooserViewModel->RefreshTableColumnsDelegate = FRefreshTable::CreateLambda([WeakPtr = AsWeak()]()
		{
			if (TSharedPtr<SWidget> Ptr = WeakPtr.Pin())
			{
				StaticCastSharedPtr<SChooserTableWidget>(Ptr)->UpdateTableColumns();
			}
		});

		ChooserViewModel->SelectRowsDelegate = FSelectRows::CreateLambda([WeakPtr = AsWeak()](TConstArrayView<int32> Rows)
		{
			if (TSharedPtr<SWidget> Ptr = WeakPtr.Pin())
			{
				StaticCastSharedPtr<SChooserTableWidget>(Ptr)->SelectRows(Rows);
			}
		});

		CreateColumnComboButton = SNew(SChooserCreateColumnButton).ViewModel(ChooserViewModel);
		
		HeaderRow = SNew(SHeaderRow);
		UpdateTableColumns();

		TSharedRef VerticalScrollbar = SNew(SScrollBar);
		
		TableView = SNew(SListView<TSharedPtr<FChooserTableRow>>)
					.OnKeyDownHandler_Lambda([Commands = InArgs._Commands](const FGeometry&, const FKeyEvent& Event)
					{
						if (Commands)
						{
							if (Commands->ProcessCommandBindings(Event))
							{
								return FReply::Handled();
							}
						}
						return FReply::Unhandled();
					})
					.ListItemsSource(&TableRows)
					.OnContextMenuOpening(this, &SChooserTableWidget::GenerateRowContextMenu)
					.OnGenerateRow(this, &SChooserTableWidget::GenerateTableRow)
					.HeaderRow(HeaderRow)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
					.ExternalScrollbar(VerticalScrollbar)
					.OnSelectionChanged_Lambda([this](TSharedPtr<FChooserTableRow>,  ESelectInfo::Type SelectInfo)
    				{
						TArray<TSharedPtr<FChooserTableRow>> SelectedItems;
						GetSelectedItems(SelectedItems);
						TArray<int32> SelectedRows;
						for (TSharedPtr<FChooserTableRow> RowItem : SelectedItems)
						{
							SelectedRows.Add(RowItem->RowIndex);
						}
						ChooserViewModel->SetSelectedRows(SelectedRows);
    				});

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SScrollBox).Orientation(Orient_Horizontal)
					+ SScrollBox::Slot().FillContentSize(1.0f)
				[
					TableView.ToSharedRef()
				]
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				VerticalScrollbar
			]
		];
	}

	SChooserTableWidget::~SChooserTableWidget()
	{
	}


	bool SChooserTableWidget::HasSelection() const
	{
		return ChooserViewModel->HasSelection();
	}

	bool SChooserTableWidget::HasRowsSelected() const
	{
		return ChooserViewModel->HasRowsSelected();
	}

	bool SChooserTableWidget::HasColumnSelected() const
	{
		return ChooserViewModel->HasColumnSelected();
	}

	void SChooserTableWidget::SelectRootProperties()
	{
		ChooserViewModel->SelectRootProperties();
	}

	bool SChooserTableWidget::CanAutoPopulateSelection() const 
	{
		if (HasColumnSelected())
		{
			return ChooserViewModel->CanAutoPopulateColumn(GetSelectedColumn());
		}
		else if (HasRowsSelected())
		{
			return ChooserViewModel->CanAutoPopulateRows();	
		}
		return false;
	}

	void SChooserTableWidget::AutoPopulateSelection()
	{
		if (HasColumnSelected())
		{
			ChooserViewModel->AutoPopulateColumn(GetSelectedColumn());
		}
		else if (HasRowsSelected())
		{
			ChooserViewModel->AutoPopulateRows(ChooserViewModel->GetSelectedRows());
		}
	}

	void SChooserTableWidget::AutoPopulateAll()
	{
		ChooserViewModel->AutoPopulateAll();
	}
	
	void SChooserTableWidget::RemoveDisabledData()
	{
		ChooserViewModel->RemoveDisabledData();
	}


	void SChooserTableWidget::DeleteSelection()
	{
		if (HasColumnSelected())
		{
			ChooserViewModel->DeleteColumn(GetSelectedColumn());
			SelectRootProperties();
		}
		else if (HasRowsSelected())
		{
			ChooserViewModel->DeleteRows(ChooserViewModel->GetSelectedRows());
		}
	}

	void SChooserTableWidget::DuplicateSelection()
	{
		if (HasRowsSelected())
		{
			ChooserViewModel->DuplicateRows(ChooserViewModel->GetSelectedRows());
		}
		else if(HasColumnSelected())
		{
			ChooserViewModel->DuplicateColumn(GetSelectedColumn());
		}
	}
	
	bool SChooserTableWidget::IsSelectionDisabled() const
	{
		if (HasRowsSelected())
		{
			return ChooserViewModel->AreRowsDisabled(ChooserViewModel->GetSelectedRows());
		}
		else if(HasColumnSelected())
		{
			return ChooserViewModel->IsColumnDisabled(GetSelectedColumn());
		}
		return false;
	}

	void SChooserTableWidget::ToggleDisableSelection()
	{
		if (HasRowsSelected())
		{
			ChooserViewModel->ToggleDisableRows(ChooserViewModel->GetSelectedRows());
		}
		else if(HasColumnSelected())
		{
			ChooserViewModel->ToggleDisableColumn(GetSelectedColumn());
		}
	}
	
	void SChooserTableWidget::CopySelection()
	{
		if (HasRowsSelected())
		{
			ChooserViewModel->CopyRows(ChooserViewModel->GetSelectedRows());
		}
		else if(HasColumnSelected())
		{
			ChooserViewModel->CopyColumn(GetSelectedColumn());
		}
	}

	void SChooserTableWidget::CutSelection()
	{
		if (HasRowsSelected())
		{
			TConstArrayView<int32> SelectedRows = ChooserViewModel->GetSelectedRows();
			ChooserViewModel->CopyRows(SelectedRows);
			ChooserViewModel->DeleteRows(SelectedRows);
		}
		else if(HasColumnSelected())
		{
			int ColumnIndex = GetSelectedColumn();
			ChooserViewModel->CopyColumn(ColumnIndex);
			ChooserViewModel->DeleteColumn(ColumnIndex);
		}
	}

	bool SChooserTableWidget::CanPaste() const
	{
		return ChooserViewModel->CanPaste();
	}

	void SChooserTableWidget::Paste()
	{
		ChooserViewModel->Paste();
	}

	
	bool SChooserTableWidget::CanMoveRowsUp()
	{
		return ChooserViewModel->CanMoveRowsUp(ChooserViewModel->GetSelectedRows());
	}

	void SChooserTableWidget::MoveRowsUp()
	{
		if (HasRowsSelected())
		{
			TConstArrayView<int32> SelectedRows = ChooserViewModel->GetSelectedRows();

			int MinSelectedRow = ChooserViewModel->GetNumRows();
			for(int SelectedRow : SelectedRows)
			{
				if (SelectedRow != ColumnWidget_SpecialIndex_Fallback)
				{
					MinSelectedRow = FMath::Min(SelectedRow, MinSelectedRow);
				}
			}
			ChooserViewModel->MoveRows(SelectedRows, MinSelectedRow - 1);
		}
	}

	bool SChooserTableWidget::CanMoveRowsDown()
	{
		return ChooserViewModel->CanMoveRowsDown(ChooserViewModel->GetSelectedRows());
	}

	void SChooserTableWidget::MoveRowsDown()
	{
		if (HasRowsSelected())
		{
			TConstArrayView<int32> SelectedRows =	ChooserViewModel->GetSelectedRows();

			int MaxSelectedRow = -1;
			for(int SelectedRow : SelectedRows)
			{
				MaxSelectedRow = FMath::Max(SelectedRow, MaxSelectedRow);
			}
			ChooserViewModel->MoveRows(SelectedRows, MaxSelectedRow + 2);
		}
	}

	bool SChooserTableWidget::CanMoveColumnLeft()
	{
		return ChooserViewModel->CanMoveColumnLeft(GetSelectedColumn());
	}

	void SChooserTableWidget::MoveColumnLeft()
	{
		ChooserViewModel->MoveColumnLeft(GetSelectedColumn());
	}

	bool SChooserTableWidget::CanMoveColumnRight()
	{
		return ChooserViewModel->CanMoveColumnRight(GetSelectedColumn());
	}

	void SChooserTableWidget::MoveColumnRight()
	{
		ChooserViewModel->MoveColumnRight(GetSelectedColumn());
	}

	void SChooserTableWidget::BindCommands(TSharedPtr<FUICommandList> InCommandList)
	{
		CommandList = InCommandList;

		const FChooserTableEditorCommands& Commands = FChooserTableEditorCommands::Get();

		CommandList->MapAction(
			Commands.EditChooserSettings,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::SelectRootProperties));
		
		CommandList->MapAction(
			Commands.AutoPopulateAll,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::AutoPopulateAll));
		
		CommandList->MapAction(
			Commands.RemoveDisabledData,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::RemoveDisabledData));
		
		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::DeleteSelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection)
			);

		CommandList->MapAction(
			FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::DuplicateSelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection)
			);

		CommandList->MapAction(
			Commands.AutoPopulateSelection,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::AutoPopulateSelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanAutoPopulateSelection),
			FGetActionCheckState(),FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection)
			);
		
		CommandList->MapAction(
			Commands.Disable,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::ToggleDisableSelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection),
			FIsActionChecked::CreateSP(SharedThis(this), &SChooserTableWidget::IsSelectionDisabled),
			FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection)
			);

		CommandList->MapAction(
			FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CopySelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection));
		
		CommandList->MapAction(
			FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CutSelection),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::HasSelection));
		
		CommandList->MapAction(
			FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::Paste),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanPaste));
		
		CommandList->MapAction(
			Commands.MoveUp,
			FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::MoveRowsUp),
			FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanMoveRowsUp),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasRowsSelected)
			);
		
		CommandList->MapAction(
				Commands.MoveDown,
				FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::MoveRowsDown),
				FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanMoveRowsDown),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasRowsSelected)
				);
		
		CommandList->MapAction(
				Commands.MoveLeft,
				FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::MoveColumnLeft),
				FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanMoveColumnLeft),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasColumnSelected)
				);

		CommandList->MapAction(
				Commands.MoveRight,
				FExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::MoveColumnRight),
				FCanExecuteAction::CreateSP(SharedThis(this), &SChooserTableWidget::CanMoveColumnRight),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(SharedThis(this), &SChooserTableWidget::HasColumnSelected)
				);
	}

}

#undef LOCTEXT_NAMESPACE
