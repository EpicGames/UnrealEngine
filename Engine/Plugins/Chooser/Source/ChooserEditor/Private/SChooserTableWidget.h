// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Commands/UICommandList.h"
#include "ChooserTableViewModel.h"
#include "IChooserTableView.h"

class SPositiveActionButton;

namespace UE::ChooserEditor
{
	
class SChooserTableWidget : public IChooserTableView
{
public:
	SLATE_BEGIN_ARGS(SChooserTableWidget)
	{}
	SLATE_ARGUMENT(TSharedPtr<FUICommandList>, Commands)
	SLATE_ARGUMENT(TSharedPtr<IChooserTableViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SChooserTableWidget() override;
	
	void SelectRows(const TConstArrayView<int32> Rows);
	void SelectRow(int32 RowIndex, bool bClear = true);

	void UpdateTableColumns();
	void UpdateTableRows();

	void ClearSelection();
	void GetSelectedItems(TArray<TSharedPtr<FChooserTableRow>> &SelectedItems);

	int GetSelectedColumn() const;

	bool TableHasFocus() const;
	
	// Command handler functions
	
	bool HasSelection() const;
	bool HasRowsSelected() const;
	bool HasColumnSelected() const;
	
	void SelectRootProperties();
	
	bool CanAutoPopulateSelection() const;
	void AutoPopulateSelection();
	void AutoPopulateAll();
	
	void RemoveDisabledData();
	void DeleteSelection();
	void DuplicateSelection();
	
	bool IsSelectionDisabled() const;
	void ToggleDisableSelection();
	
	void CopySelection();
	void CutSelection();
	bool CanPaste() const;
	void Paste();
	
	bool CanMoveRowsUp();
	void MoveRowsUp();
	bool CanMoveRowsDown();
	void MoveRowsDown();
	bool CanMoveColumnLeft();
	void MoveColumnLeft();
	bool CanMoveColumnRight();
	void MoveColumnRight();

	
	virtual TSharedPtr<IChooserTableViewModel> GetViewModel() override { return StaticCastSharedPtr<IChooserTableViewModel>(ChooserViewModel); };
	
 private:
	void BindCommands(TSharedPtr<FUICommandList> CommandList);

	TSharedPtr<SListView<TSharedPtr<FChooserTableRow>>> TableView;
	TSharedPtr<SHeaderRow> HeaderRow;
	
	TSharedPtr<SWidget> CreateColumnComboButton;
	TSharedPtr<SWidget> GenerateRowContextMenu();
	TSharedRef<ITableRow> GenerateTableRow(TSharedPtr<FChooserTableRow> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<FUICommandList> CommandList;

	TSharedPtr<FChooserTableViewModel> ChooserViewModel;
	TArray<TSharedPtr<FChooserTableRow>> TableRows;
};

}
