// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNestedChooserTree.h"
#include "ChooserTableEditor.h"
#include "Chooser.h"
#include "Framework/Application/SlateApplication.h"
#include "ObjectTools.h"
#include "OutputObjectColumn.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/STextEntryPopup.h"

#define LOCTEXT_NAMESPACE "NestedChooserTree"

namespace UE::ChooserEditor
{
	
	TSharedRef<ITableRow> SNestedChooserTree::TreeViewGenerateRow(TSharedPtr<FNestedChooserTreeEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
			return SNew(STableRow<TSharedPtr<FNestedChooserTreeEntry>>, OwnerTable)
						.Content()
						[
							SNew(STextBlock).Text(FText::FromString(InItem->Object->GetName()))
						];
	}
	
	void SNestedChooserTree::TreeViewGetChildren(TSharedPtr<FNestedChooserTreeEntry> InItem, TArray<TSharedPtr<FNestedChooserTreeEntry>>& OutChildren)
    {
		for(TSharedPtr<FNestedChooserTreeEntry>& Entry : AllChoosers)
		{
			if (Entry->Object->GetOuter() == InItem->Object)
			{
				OutChildren.Add(Entry);
			}
		}
    }
	
	void SNestedChooserTree::TreeViewDoubleClicked(TSharedPtr<FNestedChooserTreeEntry> SelectedObject)
	{
		if (UChooserTable* Chooser = Cast<UChooserTable>(SelectedObject->Object))
		{
			ChooserEditor->SetChooserTableToEdit(Chooser);
		}
		else
		{
			if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSS->OpenEditorForAsset(SelectedObject->Object);
			}
		}
	}

	void SNestedChooserTree::DeleteNestedObject()
	{
		TArray<TSharedPtr<FNestedChooserTreeEntry>>SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			if (UChooserTable* ChooserToDelete = Cast<UChooserTable>(SelectedItems[0]->Object))
			{
				if (ChooserToDelete != RootChooser)
				{
					FScopedTransaction Transaction(LOCTEXT("Delete Nested Chooser", "Delete Nested Chooser"));
					RootChooser->DeleteNestedChooser(ChooserToDelete);
				}
			}
			else if (UObject* ObjectToDelete = SelectedItems[0]->Object)
			{
				FScopedTransaction Transaction(LOCTEXT("Delete Nested Object", "Delete Nested Object"));
				RootChooser->DeleteNestedObject(ObjectToDelete);
			}
		}
	}

	void SNestedChooserTree::RenameNestedObject()
	{
		TArray<TSharedPtr<FNestedChooserTreeEntry>>SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1 && SelectedItems[0])
		{
			UObject* ObjectToRename = SelectedItems[0]->Object;

			if (ObjectToRename != RootChooser)
			{
				TSharedRef<STextEntryPopup> TextEntry =
					SNew(STextEntryPopup)
					.DefaultText(FText::FromString(ObjectToRename->GetName()))
					.Label(LOCTEXT("RenameNestedObjectLabel", "Rename"))
					.OnTextCommitted_Lambda([ObjectToRename, this](FText InText, ETextCommit::Type InCommitType)
					{
						if (InCommitType == ETextCommit::OnEnter)
						{
							FSlateApplication::Get().DismissAllMenus();

							FString NewName = InText.ToString();
							
							for(UObject* NestedObject : RootChooser->NestedObjects)
							{
								if (NestedObject->GetName() == NewName)
								{
									return;
								}
							}
							
							const FScopedTransaction Transaction(LOCTEXT("Rename Nested Object", "Rename Nested Object"));
				
							ObjectToRename->Modify(true);
							ObjectToRename->Rename(*NewName);
							RefreshAll();
						}
					});

				FSlateApplication& SlateApp = FSlateApplication::Get();
				SlateApp.PushMenu(
					SlateApp.GetInteractiveTopLevelWindows()[0],
					FWidgetPath(),
					TextEntry,
					SlateApp.GetCursorPos(),
					FPopupTransitionEffect::TypeInPopup
					);
			}
		}
	}

	void SNestedChooserTree::Construct(const FArguments& InArgs)
	{
		ChooserEditor = InArgs._ChooserEditor;
		RootChooser = ChooserEditor->GetRootChooser();
		RootChooser->NestedObjectsChanged.AddRaw(this, &SNestedChooserTree::RefreshAll);
		TreeEntries.Add(MakeShared<FNestedChooserTreeEntry>(FNestedChooserTreeEntry({RootChooser})));

		TreeView = SNew(STreeView<TSharedPtr<FNestedChooserTreeEntry>>)
							.OnExpansionChanged_Lambda([](TSharedPtr<FNestedChooserTreeEntry> Entry, bool bExpanded){ Entry->bExpanded = bExpanded; })
							.OnKeyDownHandler_Lambda([this](const FGeometry&, const FKeyEvent& Event)
							{
								if (Event.GetKey() == EKeys::Delete)
								{
									DeleteNestedObject();
									return FReply::Handled();
								}
								else if (Event.GetKey() == EKeys::F2)
								{
									RenameNestedObject();
									return FReply::Handled();
								}
								return FReply::Unhandled();
							})
							.OnContextMenuOpening(this, &SNestedChooserTree::TreeViewContextMenuOpening)
							.TreeItemsSource(&TreeEntries)
							.SelectionMode(ESelectionMode::Single)
							.OnGenerateRow(this, &SNestedChooserTree::TreeViewGenerateRow)
							.OnGetChildren(this, &SNestedChooserTree::TreeViewGetChildren)
							.OnMouseButtonDoubleClick(this, &SNestedChooserTree::TreeViewDoubleClicked)
							;
		
		RefreshAll();
	
		ChildSlot
		[
			TreeView.ToSharedRef()
		];

	}

	SNestedChooserTree::~SNestedChooserTree()
	{
		if (RootChooser)
		{
			RootChooser->NestedObjectsChanged.RemoveAll(this);
		}
	}

	static TSharedPtr<FNestedChooserTreeEntry> MakeEntry(TArray<TSharedPtr<FNestedChooserTreeEntry>>& OldValues, UObject* Object)
	{
		for(TSharedPtr<FNestedChooserTreeEntry>& Entry : OldValues)
		{
			if (Entry->Object == Object)
			{
				return Entry;
			}
		}

		return MakeShared<FNestedChooserTreeEntry>(FNestedChooserTreeEntry({Object, true}));
	}

	void SNestedChooserTree::RefreshAll()
	{
		TArray<TSharedPtr<FNestedChooserTreeEntry>> OldValues = AllChoosers;
		AllChoosers.SetNum(0);
		RootChooser = ChooserEditor->GetRootChooser();
		AllChoosers.Add(MakeEntry(OldValues, RootChooser));
		for(UObject* Object : RootChooser->NestedObjects)
		{
			AllChoosers.Add(MakeEntry(OldValues, Object));
		}

		if (TreeView.IsValid())
		{
			TreeView->RebuildList();
			
			for(TSharedPtr<FNestedChooserTreeEntry>& Entry : AllChoosers)
			{
				TreeView->SetItemExpansion(Entry, Entry->bExpanded);
			}
		}
	}

	TSharedPtr<SWidget> SNestedChooserTree::TreeViewContextMenuOpening()
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		MenuBuilder.AddMenuEntry(LOCTEXT("Delete", "Delete"), LOCTEXT("Delete Tooltip", "Delete Nested Object"), FSlateIcon(), FUIAction
			(
				FExecuteAction::CreateRaw(this, &SNestedChooserTree::DeleteNestedObject)
				)
				);
		MenuBuilder.AddMenuEntry(LOCTEXT("Rename", "Rename"), LOCTEXT("Rename Tooltip", "Rename Nested Object"), FSlateIcon(), FUIAction
			(
				FExecuteAction::CreateRaw(this, &SNestedChooserTree::RenameNestedObject)
				)
				);

		return MenuBuilder.MakeWidget();
	}
}

#undef LOCTEXT_NAMESPACE
