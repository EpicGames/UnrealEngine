// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class UChooserTable;

namespace UE::ChooserEditor
{

class FChooserTableEditor;

struct FNestedChooserTreeEntry
{
	UObject* Object = nullptr;
	bool bExpanded = true;
};

class SNestedChooserTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNestedChooserTree)
	{}
	SLATE_ARGUMENT(FChooserTableEditor*, ChooserEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SNestedChooserTree() override;

	void RefreshAll();
private:
	TSharedPtr<SWidget> TreeViewContextMenuOpening();
	TSharedRef<ITableRow> TreeViewGenerateRow(TSharedPtr<FNestedChooserTreeEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void TreeViewGetChildren(TSharedPtr<FNestedChooserTreeEntry> InItem, TArray<TSharedPtr<FNestedChooserTreeEntry>>& OutChildren);
	void TreeViewDoubleClicked(TSharedPtr<FNestedChooserTreeEntry> SelectedObject);
	void DeleteNestedObject();
	void RenameNestedObject();
	
	FChooserTableEditor* ChooserEditor = nullptr;
	UChooserTable* RootChooser = nullptr;
	TSharedPtr<STreeView<TSharedPtr<FNestedChooserTreeEntry>>> TreeView;

	TArray<TSharedPtr<FNestedChooserTreeEntry>> TreeEntries;
	TArray<TSharedPtr<FNestedChooserTreeEntry>> AllChoosers;
};

}
