// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyAccessChainWidget.h"

#include "Chooser.h"
#include "ChooserPropertyAccess.h"
#include "DetailWidgetRow.h"
#include "GraphEditorSettings.h"
#include "SClassViewer.h"
#include "IPropertyAccessEditor.h"
#include "ScopedTransaction.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "PropertyAccessChainWidget"

namespace UE::ChooserEditor
{

void CreatePropertyAccessMenus(const FString& TypeFilter, FOnAddBinding OnAddBinding, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder)
{
	FString AlternateTypeFilter;
	if (TypeFilter.Len() > 0)
	{
		if (TypeFilter[TypeFilter.Len() - 1] == '*')
		{
			FString Trimmed = TypeFilter.TrimChar('*');
			AlternateTypeFilter = "TObjectPtr<" + Trimmed + ">";
		}
	}
	
	FPropertyBindingWidgetArgs Args;
	Args.bAllowPropertyBindings = true;
	Args.OnAddBinding = OnAddBinding;

	TConstArrayView<FInstancedStruct> ContextData;
	if (ContextClassOwner)
	{
		ContextData = ContextClassOwner->GetContextData();
	}
	
	Args.bAllowUObjectFunctions = true;
	Args.bAllowOnlyThreadSafeFunctions = true;

	auto CanBindProperty = [TypeFilter, AlternateTypeFilter](FProperty* Property, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			if (TypeFilter == "" || Property == nullptr)
			{
				return true;
			}
			if (TypeFilter == "struct")
			{
				// special case for struct of any type
				return CastField<FStructProperty>(Property) != nullptr;
			}
			if (TypeFilter == "object")
			{
				// special case for objects references of any type
				return CastField<FObjectPropertyBase>(Property) != nullptr;
			}
			if (TypeFilter == "double")
			{
				// special case for doubles to bind to either floats or doubles
				return Property->GetCPPType() == "float" || Property->GetCPPType() == "double" || Property->GetCPPType() == "int32";
			}
			else if (TypeFilter == "enum")
			{
				// special case for enums, to find properties of type EnumProperty or ByteProperty which have an Enum
	
				if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property))
				{
					return true;
				}
				else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(Property))
				{
					return ByteProperty->Enum != nullptr;
				}
				return false;
			}
			else if (TypeFilter == "bool")
			{
				// special case for bools, because CPPType == "bool" doesn't catch: uint8 bBool : 1
	
				if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
				{
					return true;
				}
				return false;
			}

			const FString CPPType = Property->GetCPPType();

			return CPPType == TypeFilter || CPPType == AlternateTypeFilter;
		};

	// allow struct bindings to bind context structs directly
	Args.OnCanBindToContextStructWithIndex = FOnCanBindToContextStructWithIndex::CreateLambda([TypeFilter](UStruct* StructType, int32 StructIndex)
	{
		if (StructType)
		{
			if (TypeFilter == "struct" && !StructType->IsChildOf(UObject::StaticClass()))
			{
				// struct bindings can bind any type of struct
				return true;
			}
			else
			{
				const FString CPPName = StructType->GetPrefixCPP() +  StructType->GetName();
				return CPPName  == TypeFilter;
			}
		}
		return false;
	});

	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda(CanBindProperty);

	Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda([CanBindProperty](UFunction* Function)
	{
		if (Function->NumParms !=1)
		{
			// only allow binding object member functions which have no parameters
			return false;
		}

		if (FProperty* ReturnProperty = Function->GetReturnProperty())
		{
			return CanBindProperty(ReturnProperty, {});
		}
	
		return false;
	});

	Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass)
	{
		return true;
	});

	Args.OnCanBindToSubObjectClass = FOnCanBindToSubObjectClass::CreateLambda([](UClass* InClass)
	{
		// CanBindToSubObjectClass does the opposite of what it's name says.  True means don't allow bindings
		// don't allow binding to any object propertoes (forcing use of thread safe functions to access objects)
		return true;
	});

	Args.OnCanAcceptPropertyOrChildrenWithBindingChain = FOnCanAcceptPropertyOrChildrenWithBindingChain::CreateLambda([](FProperty* InProperty, TConstArrayView<FBindingChainElement> BindingChain)
	{
		// Make only blueprint visible properties visible for binding.
		return InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible);
	});

	
	for(int ContextIndex = 0; ContextIndex < ContextData.Num(); ContextIndex++)
	{
		const FInstancedStruct &ContextStruct = ContextData[ContextIndex];
		
		bool bAllowBindRoot = false; 
		
		UStruct* Struct = nullptr;
		if (const FContextObjectTypeClass* ClassType = ContextStruct.GetPtr<FContextObjectTypeClass>())
		{
			Struct = ClassType->Class;
		}
		else if (const FContextObjectTypeStruct* StructType = ContextStruct.GetPtr<FContextObjectTypeStruct>())
		{
			Struct = StructType->Struct;
			if (Struct)
			{
				bAllowBindRoot = TypeFilter == "struct" || (Struct->GetPrefixCPP() + Struct->GetName()) == TypeFilter;
			}
		}
		else
		{
			continue;
		}

		if (Struct)
		{
			// Show struct properties.
			MenuBuilder.AddSubMenu(
			SNew(STextBlock).Text(FText::FromString(Struct->GetName())),
			FNewMenuDelegate::CreateLambda([Struct, bAllowBindRoot, ContextIndex, Args](FMenuBuilder& MenuBuilder)
				{
					// Make first chain element representing the index in the context array.
					TArray<TSharedPtr<FBindingChainElement>> BindingChain;
					BindingChain.Emplace(MakeShared<FBindingChainElement>(nullptr, ContextIndex));

					if(bAllowBindRoot)
					{
						// Add menu entry for binding a struct (eg struct output column) directly to a context entry
						MenuBuilder.AddMenuEntry(FText::FromString(Struct->GetName()), LOCTEXT("Bind directly to this context struct", "Bind directly to this context struct"),
							FSlateIcon(),
							FExecuteAction::CreateLambda([Args, ContextIndex]
								{
									TArray<FBindingChainElement> BindingChain;
									BindingChain.Add({nullptr, ContextIndex});
									Args.OnAddBinding.Execute(NAME_None, BindingChain);
								}));
						
					}
				
					const IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
					PropertyAccessEditor.FillPropertyMenu(MenuBuilder, Args, NAME_None, Struct, BindingChain);
				}));
		}
	}
    	
}
	
	
void SPropertyAccessChainWidget::SetPropertyBinding(IHasContextClass* HasContext, const TArray<FBindingChainElement>& InBindingChain, FChooserPropertyBinding& OutPropertyBinding)
{
	if (!HasContext)
	{
		return;
	}
	
	TConstArrayView<FInstancedStruct> ContextData = HasContext->GetContextData();
	Chooser::CopyPropertyChain(InBindingChain, OutPropertyBinding);

	// remap selected context index from the UI index (which skips over non object/struct context entries), to an actual index in to the context data array
	uint32 UIIndex = 0;
	for (int32 ContextIndex=0; ContextIndex<ContextData.Num(); ContextIndex++)
	{
		const FInstancedStruct& ContextStruct = ContextData[ContextIndex];
		
		if (ContextStruct.GetPtr<FContextObjectTypeClass>() || ContextStruct.GetPtr<FContextObjectTypeStruct>())
		{
			if (UIIndex == OutPropertyBinding.ContextIndex)
			{
				OutPropertyBinding.ContextIndex = ContextIndex;
				break;
			}
			UIIndex++;
		}
	}	
	
	FField* Property = InBindingChain.Last().Field.ToField();
		
	OutPropertyBinding.DisplayName = "";
	if (InBindingChain.Num() == 1)
	{
		// direct binding to a context struct/class, set display name to the struct/class name
		if (ContextData.IsValidIndex(OutPropertyBinding.ContextIndex))
		{
			if (const FContextObjectTypeStruct* StructContext = ContextData[OutPropertyBinding.ContextIndex].GetPtr<FContextObjectTypeStruct>())
			{
				if (StructContext->Struct)
				{
					OutPropertyBinding.DisplayName = StructContext->Struct->GetDisplayNameText().ToString();
				}
			}
			else if (const FContextObjectTypeClass* ClassContext = ContextData[OutPropertyBinding.ContextIndex].GetPtr<FContextObjectTypeClass>())
			{
				if (ClassContext->Class)
				{
					OutPropertyBinding.DisplayName = ClassContext->Class->GetDisplayNameText().ToString();
				}
			}
		}
	}
	else
	{
		// set displayname from property name
		if (Property)
		{
			OutPropertyBinding.DisplayName = Property->GetDisplayNameText().ToString();
			static const int ShortNameLength = 5;
			if (OutPropertyBinding.DisplayName.Len() < ShortNameLength && InBindingChain.Num() > 2)
			{
				FField* ParentProperty = InBindingChain[InBindingChain.Num() - 2].Field.ToField();
				OutPropertyBinding.DisplayName = ParentProperty->GetDisplayNameText().ToString() + "." + OutPropertyBinding.DisplayName;
			}
		}
	}

	OutPropertyBinding.SetPropertyData(HasContext,Property);
	
	OutPropertyBinding.Compile(HasContext);
}

TSharedRef<SWidget> SPropertyAccessChainWidget::CreatePropertyAccessWidget()
{
	FPropertyBindingWidgetArgs Args;
	Args.bAllowPropertyBindings = true;

	TConstArrayView<FInstancedStruct> ContextData;
	if (ContextClassOwner)
	{
		ContextData = ContextClassOwner->GetContextData();
	}
	
	Args.bAllowUObjectFunctions = true;
	Args.bAllowOnlyThreadSafeFunctions = true;

	auto CanBindProperty = [this](FProperty* Property, TConstArrayView<FBindingChainElement> InBindingChain)
	{
		if (TypeFilter == "" || Property == nullptr)
		{
			return true;
		}
		if (TypeFilter == "struct")
		{
			// special case for struct of any type
			return CastField<FStructProperty>(Property) != nullptr;
		}
		if (TypeFilter == "object")
		{
			// special case for objects references of any type
			return CastField<FObjectPropertyBase>(Property) != nullptr;
		}
		if (TypeFilter == "double")
		{
			// special case for doubles to bind to either floats or doubles
			return Property->GetCPPType() == "float" || Property->GetCPPType() == "double" || Property->GetCPPType() == "int32";
		}
		else if (TypeFilter == "enum")
		{
			// special case for enums, to find properties of type EnumProperty or ByteProperty which have an Enum
	
			if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property))
			{
				return true;
			}
			else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(Property))
			{
				return ByteProperty->Enum != nullptr;
			}
			return false;
		}
		else if (TypeFilter == "bool")
		{
			// special case for bools, because CPPType == "bool" doesn't catch: uint8 bBool : 1
	
			if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
			{
				return true;
			}
			return false;
		}

		const FString CPPType = Property->GetCPPType();

		return CPPType == TypeFilter || CPPType == AlternateTypeFilter;
	};

	// allow struct bindings to bind context structs directly
	Args.OnCanBindToContextStructWithIndex = FOnCanBindToContextStructWithIndex::CreateLambda([this](UStruct* StructType, int32 StructIndex)
	{
		if (StructType)
		{
			if (TypeFilter == "struct" && !StructType->IsChildOf(UObject::StaticClass()))
			{
				// struct bindings can bind any type of struct
				return true;
			}
			else
			{
				const FString CPPName = StructType->GetPrefixCPP() +  StructType->GetName();
				return CPPName  == TypeFilter;
			}
		}
		return false;
	});

	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda(CanBindProperty);

	Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda([CanBindProperty](UFunction* Function)
	{
		if (Function->NumParms !=1)
		{
			// only allow binding object member functions which have no parameters
			return false;
		}

		if (FProperty* ReturnProperty = Function->GetReturnProperty())
		{
			return CanBindProperty(ReturnProperty, {});
		}
	
		return false;
	});

	Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass)
	{
		return true;
	});

	FLinearColor BindingColorValue = FLinearColor::Gray;
	if (BindingColor != "")
	{
		const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();
		if (const FStructProperty* ColorProperty = FindFProperty<FStructProperty>(GraphEditorSettings->GetClass(), FName(BindingColor)))
		{
			BindingColorValue = *ColorProperty->ContainerPtrToValuePtr<FLinearColor>(GraphEditorSettings);
		}
	}

	Args.CurrentBindingColor = MakeAttributeLambda([BindingColorValue]() {
		return BindingColorValue;
	});
	
	Args.OnCanBindToSubObjectClass = FOnCanBindToSubObjectClass::CreateLambda([](UClass* InClass)
		{
			// CanBindToSubObjectClass does the opposite of what it's name says.  True means don't allow bindings
			// don't allow binding to any object propertoes (forcing use of thread safe functions to access objects)
			return true;
		});

	Args.OnCanAcceptPropertyOrChildrenWithBindingChain = FOnCanAcceptPropertyOrChildrenWithBindingChain::CreateLambda([](FProperty* InProperty, TConstArrayView<FBindingChainElement> BindingChain)
		{
			// Make only blueprint visible properties visible for binding.
			return InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible);
		});	

	if (OnAddBinding.IsBound())
	{
		Args.OnAddBinding = OnAddBinding; 
	}
	else
	{
		Args.OnAddBinding = FOnAddBinding::CreateLambda( 
			[this](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
			{
				if (PropertyBindingValue.IsSet())
				{
					FChooserPropertyBinding* ContextProperty = PropertyBindingValue.Get();
					
					UObject* TransactionObject = Cast<UObject>(ContextClassOwner);
                    
                    const FScopedTransaction Transaction(NSLOCTEXT("ContextPropertyWidget", "Change Property Binding", "Change Property Binding"));
                    TransactionObject->Modify(true);

					SetPropertyBinding(ContextClassOwner, InBindingChain, *PropertyBindingValue.Get());
					OnValueChanged.ExecuteIfBound();
				}
			});
	}

	Args.CurrentBindingToolTipText = MakeAttributeLambda([this]()
	{
		const FText Bind = NSLOCTEXT("ContextPropertyWidget", "Bind", "Bind");
		FText CurrentValue = Bind;
		
		const FChooserPropertyBinding* PropertyValue = PropertyBindingValue.Get();
		if (PropertyValue != nullptr)
		{
			if (!PropertyValue->CompileMessage.IsEmpty())
			{
				CurrentValue = PropertyValue->CompileMessage;
			}
			else
			{
				if (PropertyValue->PropertyBindingChain.Num()>0)
				{
					TArray<FText> BindingChainText;
					BindingChainText.Reserve(PropertyValue->PropertyBindingChain.Num());
			 
					for (const FName& Name : PropertyValue->PropertyBindingChain)
					{
						BindingChainText.Add(FText::FromName(Name));
					}
				
					CurrentValue = FText::Join(NSLOCTEXT("ContextPropertyWidget", "PropertyPathSeparator","."), BindingChainText);
				}
			}
		}

		return CurrentValue;	
	});

	Args.CurrentBindingText = MakeAttributeLambda([this]()
			{
				const FText Bind = NSLOCTEXT("ContextPropertyWidget", "Bind", "Bind");
				FText CurrentValue = Bind;
		
				const FChooserPropertyBinding* PropertyValue = PropertyBindingValue.Get();

				if (PropertyValue == nullptr)
				{
					return FText();
				}
	
				int BindingChainLength = PropertyValue->PropertyBindingChain.Num();
				if (BindingChainLength == 0)
				{
					if (PropertyValue->ContextIndex >= 0)
					{
						// direct binding to a context struct
						if (ContextClassOwner)
						{
							TConstArrayView<FInstancedStruct> ContextData = ContextClassOwner->GetContextData();
							if (ContextData.IsValidIndex(PropertyValue->ContextIndex))
							{
								if (const FContextObjectTypeStruct* StructType = ContextData[PropertyValue->ContextIndex].GetPtr<FContextObjectTypeStruct>())
								{
									if (StructType->Struct)
									{
										CurrentValue = FText::FromString(StructType->Struct->GetAuthoredName());
									}
								}
							}
						}
					}
				}
				else
				{
					if (!PropertyValue->DisplayName.IsEmpty())
					{
						CurrentValue = FText::FromString(PropertyValue->DisplayName);
					}
					else if (BindingChainLength == 1)
					{
						// single property, just use the property name
						CurrentValue = FText::FromName(PropertyValue->PropertyBindingChain.Last());
					}
					else
					{
						// for longer chains always show the last struct/object name, and the final property name (full path in tooltip)
						CurrentValue = FText::Join(NSLOCTEXT("ContextPropertyWidget", "PropertyPathSeparator","."),
							TArray<FText>({
								FText::FromName(PropertyValue->PropertyBindingChain[BindingChainLength-2]),
								FText::FromName(PropertyValue->PropertyBindingChain[BindingChainLength-1])
							}));
					}
				}
	
				return CurrentValue;
			});

	Args.CurrentBindingImage = MakeAttributeLambda([this]() -> const FSlateBrush*
		{
			static FName PropertyIcon(TEXT("Kismet.Tabs.Variables"));
			static FName WarningIcon(TEXT("Icons.WarningWithColor"));
			bool bHasWarning = false;
		
			const FChooserPropertyBinding* PropertyValue = PropertyBindingValue.Get();
			if (PropertyValue != nullptr)
			{
				bHasWarning = !PropertyValue->CompileMessage.IsEmpty();
			}
		
			return FAppStyle::GetBrush(bHasWarning ? WarningIcon : PropertyIcon);
		});
	
	TArray<FBindingContextStruct> ContextStructs;
	for (const FInstancedStruct& ContextStruct : ContextData)
	{
		if (const FContextObjectTypeClass* ClassType = ContextStruct.GetPtr<FContextObjectTypeClass>())
		{
			ContextStructs.SetNum(ContextStructs.Num()+1);
			ContextStructs.Last().Struct = ClassType->Class;
		}
		else if (const FContextObjectTypeStruct* StructType = ContextStruct.GetPtr<FContextObjectTypeStruct>())
		{
			ContextStructs.SetNum(ContextStructs.Num()+1);
			ContextStructs.Last().Struct = StructType->Struct;
		}
	}	

	const IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	return  PropertyAccessEditor.MakePropertyBindingWidget(ContextStructs, Args);
}

void SPropertyAccessChainWidget::UpdateWidget()
{
	ChildSlot[ CreatePropertyAccessWidget() ];
}

void SPropertyAccessChainWidget::ContextClassChanged()
{
	UpdateWidget();
}

void SPropertyAccessChainWidget::Construct( const FArguments& InArgs)
{
	TypeFilter = InArgs._TypeFilter;
	BindingColor = InArgs._BindingColor;
	ContextClassOwner = InArgs._ContextClassOwner;
	bAllowFunctions = InArgs._AllowFunctions;
	OnValueChanged = InArgs._OnValueChanged;
	PropertyBindingValue = InArgs._PropertyBindingValue;
	OnAddBinding = InArgs._OnAddBinding;
	UpdateWidget();

	if (TypeFilter.Len() > 0)
	{
		if (TypeFilter[TypeFilter.Len() - 1] == '*')
		{
			FString Trimmed = TypeFilter.TrimChar('*');
			AlternateTypeFilter = "TObjectPtr<" + Trimmed + ">";
		}
	}

	if (ContextClassOwner)
	{
		ContextClassOwner->OnContextClassChanged.AddSP(this, &SPropertyAccessChainWidget::ContextClassChanged);
	}
}

SPropertyAccessChainWidget::~SPropertyAccessChainWidget()
{
}

}

#undef LOCTEXT_NAMESPACE