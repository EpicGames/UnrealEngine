// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ObjectChooserWidgetFactories.h"
#include "Widgets/SWidget.h"

class UChooserTable;
struct FChooserColumnBase;


namespace UE::ChooserEditor
{
	CHOOSEREDITOR_API TSharedRef<SWidget> MakeColumnHeaderWidget(UChooserTable* Chooser, FChooserColumnBase* Column, const FText& ColumnName,const FText& ColumnTooltip, const FSlateBrush* ColumnIcon, TSharedPtr<SWidget> DebugWidget, FChooserWidgetValueChanged ValueChanged = FChooserWidgetValueChanged());
}