// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

namespace UE::ChooserEditor
{
	
class FChooserEditorStyle : FSlateStyleSet
{
public:
    FChooserEditorStyle();
	static void Initialize();
	static void Shutdown();

	CHOOSEREDITOR_API static const ISlateStyle& Get();

private:

	static TSharedPtr< FChooserEditorStyle > StyleInstance;
};

}
