// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chooser.h"
#include "ChooserPropertyAccess.h"

#include "ChooserInitializer.generated.h"

USTRUCT()
struct FChooserInitializer
{
	GENERATED_BODY()
	virtual ~FChooserInitializer() = default;

	virtual UClass* OverrideClass(UClass* Class) const
	{
		return Class;
	}
	
	virtual void InitializeSignature(UChooserSignature* ChooserSignature) const
	{
		// fallback codepath in case there are any old Initializers out there that haven't updated to InitalizeSignature
		if (UChooserTable* Chooser = Cast<UChooserTable>(ChooserSignature))
		{
			Initialize(Chooser);
		}
	};

	// Deprecated, implement InitializeSignature
	virtual void Initialize(UChooserTable* Chooser) const {};
};


USTRUCT(DisplayName="Generic Chooser", Meta = (ToolTip="A ChooserTable for use in Blueprint, which can return an arbitrary Asset type or Class, and can take any number of Objects or Structs as Parameters.") )
struct FGenericChooserInitializer : public FChooserInitializer
{
	GENERATED_BODY()

	virtual void InitializeSignature(UChooserSignature* ChooserSignature) const override;
	
	// The kind of output this chooser has (Object or Class)
	UPROPERTY(EditAnywhere, DisplayName = "Result Type", Category="Result")
	EObjectChooserResultType ResultType = EObjectChooserResultType::ObjectResult;
	
	
	// The Class of Object this Chooser returns when ResultType is set to ObjectOfType, or the Parent Class of the Classes returned by this chooser when ResultType is set to ClassOfType
	UPROPERTY(EditAnywhere, DisplayName= "Result Class", Category="Result", Meta = (AllowAbstract=true))
	TObjectPtr<UClass> OutputObjectType;
	
    // Parameter Objects or Structs from which the chooser can read or write properties 
	UPROPERTY(EditAnywhere, DisplayName = "Parameters", NoClear, Meta = (ExpandByDefault, ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ContextObjectTypeBase"), Category = "Parameters")
	TArray<FInstancedStruct> ContextData;
};

USTRUCT(DisplayName="Animation Chooser", meta = (ToolTip="A ChooserTable for use with the ChooserPlayer AnimGraph node.\nReturns an AnimAsset, and takes an AnimInstance, and a ChooserPlayerSettings struct as parameters."))
struct FChooserPlayerInitializer : public FChooserInitializer
{
	GENERATED_BODY()
	virtual void InitializeSignature(UChooserSignature* Chooser) const override;

	UPROPERTY(EditAnywhere, Category="AnimClass")
	TObjectPtr<UClass> AnimClass;
};

USTRUCT(DisplayName="No Primary Result Chooser", Meta = (ToolTip="A ChooserTable for use in Blueprint, which returns no primary result but writes to outputs (useful if you are interested in returning only integral types like a float or string). Note: this table can't evaluate or nest other tables, and will set its outputs to the first row that matches (no 'multi' mode).") )
struct FNoPrimaryResultChooserInitializer : public FChooserInitializer
{
	GENERATED_BODY()

	virtual void InitializeSignature(UChooserSignature* Chooser) const override;
	
	// Parameter Objects or Structs from which the chooser can read or write properties 
	UPROPERTY(EditAnywhere, DisplayName = "Parameters", NoClear, Meta = (ExpandByDefault, ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ContextObjectTypeBase"), Category = "Parameters")
	TArray<FInstancedStruct> ContextData;
};
