// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ChooserTableEditorCommands"
	
class CHOOSEREDITOR_API FChooserTableEditorCommands : public TCommands<FChooserTableEditorCommands>
{
public:
	/** Constructor */
	FChooserTableEditorCommands() 
		: TCommands<FChooserTableEditorCommands>("ChooserTableEditor", NSLOCTEXT("Contexts", "ChooserTableEditor", "Chooser Table Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	TSharedPtr<FUICommandInfo> EditChooserSettings;
	TSharedPtr<FUICommandInfo> AutoPopulateAll;
	TSharedPtr<FUICommandInfo> AutoPopulateSelection;
	TSharedPtr<FUICommandInfo> RemoveDisabledData;
	TSharedPtr<FUICommandInfo> Disable;
	TSharedPtr<FUICommandInfo> MoveLeft;
	TSharedPtr<FUICommandInfo> MoveRight;
	TSharedPtr<FUICommandInfo> MoveUp;
	TSharedPtr<FUICommandInfo> MoveDown;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(EditChooserSettings, "Table Settings", "Edit the root properties of the ChooserTable asset.", EUserInterfaceActionType::Button, FInputChord())
		UI_COMMAND(AutoPopulateAll, "AutoPopulate All", "Auto Populate cell data for all supported columns", EUserInterfaceActionType::Button, FInputChord())
		UI_COMMAND(AutoPopulateSelection, "AutoPopulate", "Auto Populate cell data for selection (requires Columns that support Auto Populate)", EUserInterfaceActionType::Button, FInputChord())
		UI_COMMAND(RemoveDisabledData, "Remove Disabled Data", "Delete all data that's marked as disabled.", EUserInterfaceActionType::Button, FInputChord())
		UI_COMMAND(Disable, "Disable", "Disable the selected Rows or Column.", EUserInterfaceActionType::Check, FInputChord())
		UI_COMMAND(MoveLeft, "Move Left", "Move the selected column to the left", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Left))
		UI_COMMAND(MoveRight, "Move Right", "Move the selected column to the right", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Right))
		UI_COMMAND(MoveUp, "Move Up", "Move the selected row(s) up", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Up))
		UI_COMMAND(MoveDown, "Move Down", "Move the selecte row(s) down", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Down))
	}
};
	
#undef LOCTEXT_NAMESPACE
