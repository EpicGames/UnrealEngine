// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IChooserTableViewModel.h"

class UChooserTable;
class SWidget;

namespace UE::ChooserEditor
{

class IChooserEditorModule : public IModuleInterface
{
public:
	virtual TSharedPtr<IChooserTableViewModel> CreateChooserTableViewModel(UChooserTable* ChooserTable) = 0;
	virtual TSharedPtr<SWidget> CreateChooserTableView(TSharedPtr<IChooserTableViewModel> ViewModel, TSharedPtr<FUICommandList> CommandList) = 0;
};

}
