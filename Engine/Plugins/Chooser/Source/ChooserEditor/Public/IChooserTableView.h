// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "IChooserTableViewModel.h"

class FUICommandList;
class UChooserTable;

namespace UE::ChooserEditor
{

class IChooserTableView : public SCompoundWidget
{
public:
	virtual TSharedPtr<IChooserTableViewModel> GetViewModel() = 0;
};

}
