// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Misc/NotifyHook.h"
#include "ToolMenus.h"

#include "IChooserTableViewModel.generated.h"

class FUICommandList;
class UChooserTable;

namespace UE::ChooserEditor
{
	

DECLARE_DELEGATE_OneParam(FShowDetails, const TArray<UObject*>&);
DECLARE_DELEGATE_OneParam(FOpenObject, UObject*);

class CHOOSEREDITOR_API IChooserTableViewModel : public TSharedFromThis<IChooserTableViewModel>, public FNotifyHook
{
public:
	virtual ~IChooserTableViewModel() {}

	virtual void SetShowDetailsDelegate(FShowDetails InShowDetailsDelegate) = 0;
	virtual void SetOpenObjectDelegate(FOpenObject InOpenObjectDelegate) = 0;
	virtual void RegisterMenus(TSharedPtr<FUICommandList> CommandList) = 0;

	virtual const UChooserTable* GetRootChooser() const = 0;
	virtual UChooserTable* GetRootChooser() = 0;
	virtual UChooserTable* GetChooser() = 0;
	virtual const UChooserTable* GetChooser() const = 0;
	virtual void SetChooser(UChooserTable* Chooser) = 0;

	virtual void RefreshAll() = 0;
	virtual void AutoPopulateAll() = 0;
	virtual void SelectRootProperties() = 0;
	
	static const FName ChooserToolbarName;
};

}
	
UCLASS()
class UChooserEditorToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<UE::ChooserEditor::IChooserTableViewModel> ViewModel;
};