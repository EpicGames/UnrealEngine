// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClassViewerFilter.h"
#include "StructViewerFilter.h"

namespace UE::ChooserEditor
{
	// Filter class for things which implement a particular interface
	class FInterfaceClassFilter : public IClassViewerFilter
	{
	public:
		FInterfaceClassFilter(UClass* InInterfaceType) : InterfaceType(InInterfaceType)  { };
		virtual ~FInterfaceClassFilter() override = default;
		
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs ) override
		{
			return (InClass->ImplementsInterface(InterfaceType) && InClass->HasAnyClassFlags(CLASS_Abstract) == false);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return false;
		}
	private:
		UClass* InterfaceType;
	};
	
	class FStructFilter : public IStructViewerFilter
	{
	public:
		FStructFilter(const UScriptStruct* InBaseType) : BaseType(InBaseType)  { };
		
		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			if (InStruct->HasMetaData(TEXT("Hidden")))
			{
				return false;
			}
			
			return InStruct->IsChildOf(BaseType) && InStruct !=BaseType;
		}
		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs)
		{
			return false;
		};
	private:
		const UScriptStruct* BaseType;
	};
		
}
