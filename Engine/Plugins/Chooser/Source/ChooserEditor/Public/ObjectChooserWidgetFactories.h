// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chooser.h"
#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "StructViewerModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define UE_API CHOOSEREDITOR_API

class UChooserTable;
struct FChooserColumnBase;

namespace UE::ChooserEditor
{
	enum {  ColumnWidget_SpecialIndex_Header = -1 };
	enum {  ColumnWidget_SpecialIndex_Fallback = -2 };
	
	class IChooserTableWidgetInterface
	{
	public:
		virtual ~IChooserTableWidgetInterface() {}
		virtual void OpenObject(UObject* Object) = 0;
	};
	
	DECLARE_DELEGATE(FChooserWidgetValueChanged)
	typedef TFunction<TSharedRef<SWidget>(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged,  IChooserTableWidgetInterface* ChooserTableWidgetInterface)> FNewChooserWidgetCreator;
	typedef TFunction<TSharedRef<SWidget>(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, FChooserWidgetValueChanged)> FChooserWidgetCreator;
	typedef TFunction<TSharedRef<SWidget>(UChooserTable* Chooser, FChooserColumnBase* Column, int Row)> FColumnWidgetCreator;
	
	typedef TFunction<void(UObject* TransactionObject, const IHasContextClass* ContextOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged)> FParameterMenuCreator;

	class FObjectChooserWidgetFactories
	{
	public:

		static UE_API TSharedPtr<SWidget> CreateWidget(bool bReadOnly, UObject* TransactionObject, UScriptStruct* DataBaseType, FInstancedStruct* Data, UClass* ResultBaseClass,
												FChooserWidgetValueChanged ValueChanged = FChooserWidgetValueChanged(), IChooserTableWidgetInterface* ChooserWidgetInterface = nullptr, FText NullValueDisplayText = FText());
		
		static UE_API TSharedPtr<SWidget> CreateWidget(bool ReadOnly, UObject* TransactionObject, void* Value, const UStruct* ValueType, UClass* ResultBaseClass, FChooserWidgetValueChanged ValueChanged = FChooserWidgetValueChanged(),
										IChooserTableWidgetInterface* ChooserWidgetInterface = nullptr);
		
		static UE_API TSharedPtr<SWidget> CreateWidget(bool ReadOnly, UObject* TransactionObject, const UScriptStruct* BaseType, void* Value, const UStruct* ValueType, UClass* ResultBaseClass,
										const FOnStructPicked& CreateClassCallback, TSharedPtr<SBorder>* InnerWidget = nullptr, FChooserWidgetValueChanged ValueChanged = FChooserWidgetValueChanged(),
										IChooserTableWidgetInterface* ChooserWidgetInterface = nullptr,
										FText NullValueDisplayText = FText());
		
		static UE_API TSharedPtr<SWidget> CreateColumnWidget(FChooserColumnBase* Column, const UStruct* ColumnType, UChooserTable* Chooser, int Row); // todo: chooser should be a UObject
		
		static UE_API void RegisterWidgets();

		static UE_API void RegisterWidgetCreator(const UStruct* Type, FNewChooserWidgetCreator Creator);
		static UE_API void RegisterWidgetCreator(const UStruct* Type, FChooserWidgetCreator Creator);
		static UE_API void RegisterColumnWidgetCreator(const UStruct* ColumnType, FColumnWidgetCreator Creator);
		
		static UE_API void RegisterParameterMenuCreator(const UStruct* ParameterBaseType, FParameterMenuCreator);
		static UE_API void CreateParametersMenu(const UStruct* ParameterBaseType, UObject* TransactionObject, const IHasContextClass* ContextOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void()> BindingChanged);
		
	private:
		static UE_API TMap<const UStruct*, FColumnWidgetCreator>  ColumnWidgetCreators;
		static UE_API TMap<const UStruct*, FChooserWidgetCreator> ChooserWidgetCreators;
		static UE_API TMap<const UStruct*, FNewChooserWidgetCreator> NewChooserWidgetCreators;

		typedef TMultiMap<const UStruct*, FParameterMenuCreator> FParameterMenuCreatorMap;
		
		static FParameterMenuCreatorMap ParameterMenuCreators;
	};
}

#undef UE_API
