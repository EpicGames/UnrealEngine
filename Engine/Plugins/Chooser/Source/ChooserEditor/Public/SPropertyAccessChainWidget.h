// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyAccessEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/SCompoundWidget.h"
#include "IHasContext.h"
#include "ChooserPropertyAccess.h"
#include "ScopedTransaction.h"

#define UE_API CHOOSEREDITOR_API

namespace UE::ChooserEditor
{

DECLARE_DELEGATE(FPropertyAccessChainChanged)

		
// Wrapper widget for Property access widget, which can update when the target Class changes
class SPropertyAccessChainWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyAccessChainWidget)
	{}

	SLATE_ARGUMENT(IHasContextClass*, ContextClassOwner)
	SLATE_ARGUMENT(FString, TypeFilter);
	SLATE_ARGUMENT(FString, BindingColor);
	SLATE_ARGUMENT(bool, AllowFunctions);
	SLATE_EVENT(FOnAddBinding, OnAddBinding);
	SLATE_EVENT(FPropertyAccessChainChanged, OnValueChanged);
	SLATE_ATTRIBUTE(FChooserPropertyBinding*, PropertyBindingValue);
            
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs);
	UE_API virtual ~SPropertyAccessChainWidget();
	
	// Assign result from Property Access Widget to an FChooserPropertyBinding (Helper for OnAddBinding implementations)
	static UE_API void SetPropertyBinding(IHasContextClass* HasContext, const TArray<FBindingChainElement>& InBindingChain, FChooserPropertyBinding& OutPropertyBinding);

private:
	UE_API TSharedRef<SWidget> CreatePropertyAccessWidget();
	UE_API void UpdateWidget();
	UE_API void ContextClassChanged();
		
	FString TypeFilter;
	FString AlternateTypeFilter;
	FString BindingColor;
	UE_API FDelegateHandle ContextClassChangedHandle();
	IHasContextClass* ContextClassOwner = nullptr;
	FOnAddBinding OnAddBinding;
	FPropertyAccessChainChanged OnValueChanged;
	TAttribute<FChooserPropertyBinding*> PropertyBindingValue;
	bool bAllowFunctions;
};

void CreatePropertyAccessMenus(const FString& TypeFilter, FOnAddBinding OnAddBinding, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder);

}



template <typename PropertyType>
void CreatePropertyAccessMenus(const FString& TypeFilter, UObject* TransactionObject, const IHasContextClass* ContextClassOwner, FInstancedStruct* Parameter, FMenuBuilder& MenuBuilder, TFunction<void ()> BindingChanged)
{
	FOnAddBinding OnAddBinding  = FOnAddBinding::CreateLambda( 
	[Parameter, TransactionObject, ContextClassOwner, BindingChanged](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
	{
		if (Parameter)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("ContextPropertyWidget", "Change Property Binding", "Change Property Binding"));
			TransactionObject->Modify(true);
			
			if (!Parameter->GetPtr<PropertyType>())
			{
				Parameter->InitializeAs(PropertyType::StaticStruct());
			}

			PropertyType& Property = Parameter->GetMutable<PropertyType>();

			UE::ChooserEditor::SPropertyAccessChainWidget::SetPropertyBinding(const_cast<IHasContextClass*>(ContextClassOwner), InBindingChain, Property.Binding);
			if (BindingChanged.IsSet())
			{
				BindingChanged();
			}
				
				
		}
	});
	
	UE::ChooserEditor::CreatePropertyAccessMenus(TypeFilter, OnAddBinding, ContextClassOwner, Parameter, MenuBuilder);	
}

#undef UE_API
