// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChooserUncooked : ModuleRules
	{
		public ChooserUncooked(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[] {"Chooser"});
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"AssetTools",
					"UnrealEd",
					"AssetDefinition",
					"GameplayTags",
					"BlueprintGraph",
					"KismetCompiler",
					"AnimGraph",
					"AnimGraphRuntime",
					"BlendStackEditor",
					"BlendStack"
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}
