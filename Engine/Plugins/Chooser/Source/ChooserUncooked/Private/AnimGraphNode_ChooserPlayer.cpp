// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ChooserPlayer.h"

#include "EditorCategoryUtils.h"
#include "DetailLayoutBuilder.h"
#include "Animation/AnimAttributes.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimRootMotionProvider.h"
#include "ChooserPropertyAccess.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ChooserPlayer)

#define LOCTEXT_NAMESPACE "AnimGraphNode_ChooserPlayer"

void UAnimGraphNode_ChooserPlayer::UpdateContextData()
{
	if (Node.ChooserContextDefinition.IsEmpty())
	{
		Node.ChooserContextDefinition.SetNum(2);
		Node.ChooserContextDefinition[0].InitializeAs(FContextObjectTypeClass::StaticStruct());
		FContextObjectTypeClass& ClassEntry = Node.ChooserContextDefinition[0].GetMutable<FContextObjectTypeClass>();
		ClassEntry.Class = GetBlueprint()->GeneratedClass;
		ClassEntry.Direction = EContextObjectDirection::ReadWrite;
		Node.ChooserContextDefinition[1].InitializeAs(FContextObjectTypeStruct::StaticStruct());
		FContextObjectTypeStruct& StructEntry = Node.ChooserContextDefinition[1].GetMutable<FContextObjectTypeStruct>();
		StructEntry.Struct = FChooserPlayerSettings::StaticStruct();
		ClassEntry.Direction = EContextObjectDirection::ReadWrite;
	}
}

void UAnimGraphNode_ChooserPlayer::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.DefaultSettings.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

void UAnimGraphNode_ChooserPlayer::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	UpdateContextData();
}

void UAnimGraphNode_ChooserPlayer::PostPasteNode()
{
	Super::PostPasteNode();
	Node.ChooserContextDefinition.Empty();
	UpdateContextData();
}

FLinearColor UAnimGraphNode_ChooserPlayer::GetNodeTitleColor() const
{
	return FLinearColor(0.10f, 0.60f, 0.12f);
}

FText UAnimGraphNode_ChooserPlayer::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Selects Animation Assets using a Chooser, and plays them with an underlying BlendStack.");
}

FText UAnimGraphNode_ChooserPlayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.Chooser.IsValid())
	{
		FString Name;
		Node.Chooser.Get<FObjectChooserBase>().GetDebugName(Name);
		return FText::Format(LOCTEXT("EvaluateChooser2_TitleWithChooser", "Chooser Player: {0}"), {FText::FromString(Name)});
	}
	return LOCTEXT("NodeTitle", "Chooser Player");
}

FText UAnimGraphNode_ChooserPlayer::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Sequences");
}

void UAnimGraphNode_ChooserPlayer::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (FObjectChooserBase* ChooserBase = Node.Chooser.GetMutablePtr<FObjectChooserBase>())
	{
		ChooserBase->Compile(this, true);
		FText Message;
		if (ChooserBase->HasCompileErrors(Message))
		{
			FText NodeMessage = FText::Format(LOCTEXT("error in node", "{0} in @@"), Message);
			MessageLog.Error(*NodeMessage.ToString(), this);
		}
	}
	else
	{
		MessageLog.Error(TEXT("No Chooser set in @@"), this);
	}
	
}

void UAnimGraphNode_ChooserPlayer::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	if (!UE::Anim::IPoseSearchProvider::IsAvailable())
	{
		DetailBuilder.HideCategory(TEXT("PoseMatching"));
	}
}

void UAnimGraphNode_ChooserPlayer::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);

	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

#undef LOCTEXT_NAMESPACE
