// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace UE::Chooser
{

class FChooserUncookedModule : public IModuleInterface
{
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

}