// Copyright Epic Games, Inc. All Rights Reserved.


#include "EvaluateChooserNode.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "ChooserFunctionLibrary.h"
#include "ChooserPropertyAccess.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "IHasContext.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet/KismetSystemLibrary.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "K2Node_Self.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EvaluateChooserNode)

#define LOCTEXT_NAMESPACE "EvaluateChooserNode"

/////////////////////////////////////////////////////
// UK2Node_EvaluateChooser
// old implementation of this node for backwards compatibility - not currently accessible to create new instances in content

UK2Node_EvaluateChooser::UK2Node_EvaluateChooser(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Evaluates an Chooser Table, and returns the resulting Object or Objects.");
}

void UK2Node_EvaluateChooser::UnregisterChooserCallback()
{
	if(CurrentCallbackChooser)
	{
		CurrentCallbackChooser->OnOutputObjectTypeChanged.RemoveAll(this);
		CurrentCallbackChooser = nullptr;
	}
}

void UK2Node_EvaluateChooser::BeginDestroy()
{
	UnregisterChooserCallback();
	
	Super::BeginDestroy();
}

void UK2Node_EvaluateChooser::PostEditUndo()
{
	Super::PostEditUndo();
	ChooserChanged();
}

void UK2Node_EvaluateChooser::DestroyNode()
{
	UnregisterChooserCallback();
	Super::DestroyNode();
}


void UK2Node_EvaluateChooser::ChooserChanged()
{
	if (Chooser != CurrentCallbackChooser)
	{
		UnregisterChooserCallback();
	
		if (Chooser)
		{
			Chooser->OnOutputObjectTypeChanged.AddUObject(this, &UK2Node_EvaluateChooser::ResultTypeChanged);
		}
	
		CurrentCallbackChooser = Chooser;

		AllocateDefaultPins();
	}
}

void UK2Node_EvaluateChooser::ResultTypeChanged(const UClass*)
{
	AllocateDefaultPins();
}

void UK2Node_EvaluateChooser::PreloadRequiredAssets()
{
	if (Chooser)
	{
		Chooser->ConditionalPreload();
	}
    		
	Super::PreloadRequiredAssets();
}

void UK2Node_EvaluateChooser::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	
	UClass* ChooserResultType = UObject::StaticClass();

   	if (Chooser && Chooser->OutputObjectType)
   	{
   		ChooserResultType = Chooser->OutputObjectType;
   	}

	if (UEdGraphPin* ResultPin = FindPin(TEXT("Result"), EGPD_Output))
	{
		ResultPin->PinType.PinSubCategoryObject = ChooserResultType;
		ResultPin->PinType.ContainerType = (Mode == EEvaluateChooserMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
	}
	else
	{
		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.ContainerType = (Mode == EEvaluateChooserMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
		PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Object;
				
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, ChooserResultType, TEXT("Result"), PinParams);
	}
}

FText UK2Node_EvaluateChooser::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EvaluateChooser_Title", "Evaluate Chooser (Deprecated)");
}

FText UK2Node_EvaluateChooser::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	return FText::FromName(Pin->PinName);
}

void UK2Node_EvaluateChooser::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetName() == "Chooser")
		{
			ChooserChanged();
		}
		if (PropertyChangedEvent.Property->GetName() == "Mode")
		{
			AllocateDefaultPins();
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_EvaluateChooser::PostLoad()
{
	Super::PostLoad();
	ChooserChanged();
}

void UK2Node_EvaluateChooser::PinConnectionListChanged(UEdGraphPin* Pin)
{

	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_EvaluateChooser::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
}

void UK2Node_EvaluateChooser::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);
}

FText UK2Node_EvaluateChooser::GetTooltipText() const
{
	return NodeTooltip;
}

void UK2Node_EvaluateChooser::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_EvaluateChooser::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* ResultPin = FindPinChecked(TEXT("Result"));
	if (ResultPin->HasAnyConnections())
	{
		UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, this);

		if (Mode == EEvaluateChooserMode::AllResults)
		{
			CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateChooserMulti)));
		}
		else
		{
			CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateChooser)));
		}
		CallFunction->AllocateDefaultPins();

		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(SelfNode, this);
		SelfNode->AllocateDefaultPins();

		SelfNode->FindPin(TEXT("self"))->MakeLinkTo(CallFunction->FindPin(TEXT("ContextObject"))); // add null check + error
		
		UEdGraphPin* ChooserTablePin = CallFunction->FindPin(TEXT("ChooserTable"));
		CallFunction->GetSchema()->TrySetDefaultObject(*ChooserTablePin, Chooser);

		UEdGraphPin* OutputPin = CallFunction->GetReturnValuePin();

		if (Chooser && Chooser->OutputObjectType)
		{
			UEdGraphPin* OutputClassPin = CallFunction->FindPin(TEXT("ObjectClass"));
			CallFunction->GetSchema()->TrySetDefaultObject(*OutputClassPin, Chooser->OutputObjectType);
		}

		CompilerContext.MovePinLinksToIntermediate(*ResultPin, *OutputPin);
	}
	
	BreakAllNodeLinks();
}

UK2Node::ERedirectType UK2Node_EvaluateChooser::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_EvaluateChooser::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_EvaluateChooser::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EvaluateChooser::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Animation);
}


//----------------------------------------------------------------------------------------------
// UK2Node_EvaluateChooser2
// New Implementation of EvaluateChooser with support for passing in/out multiple objects and structs

UK2Node_EvaluateChooser2::UK2Node_EvaluateChooser2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("EvaluateChooser2Tooltip", "Evaluates an Chooser Table, and returns the resulting Object or Objects.");
}

void UK2Node_EvaluateChooser2::UnregisterChooserCallback()
{
	if(CurrentCallbackChooser)
	{
		CurrentCallbackChooser->OnOutputObjectTypeChanged.RemoveAll(this);
		CurrentCallbackChooser->OnContextClassChanged.RemoveAll(this);
		CurrentCallbackChooser = nullptr;
	}
}

void UK2Node_EvaluateChooser2::BeginDestroy()
{
	UnregisterChooserCallback();
	
	Super::BeginDestroy();
}

void UK2Node_EvaluateChooser2::PostEditUndo()
{
	Super::PostEditUndo();
	ChooserChanged();
}

void UK2Node_EvaluateChooser2::DestroyNode()
{
	UnregisterChooserCallback();
	Super::DestroyNode();
}


void UK2Node_EvaluateChooser2::ChooserChanged()
{
	if (Chooser != CurrentCallbackChooser)
	{
		UnregisterChooserCallback();
	
		bShowModeSetting = false;
		
		if (Chooser)
		{
			Chooser->OnOutputObjectTypeChanged.AddUObject(this, &UK2Node_EvaluateChooser2::ResultTypeChanged);
			Chooser->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
			
			if (Chooser->ResultType != EObjectChooserResultType::NoPrimaryResult)
			{
				bShowModeSetting = true;
			}
			else
			{
				// NoPrimaryResult tables only set the output to their first result anyway
				Mode = EEvaluateChooserMode::FirstResult;
			}
		}
	
		CurrentCallbackChooser = Chooser;

		ReconstructNode();
	}
}

void UK2Node_EvaluateChooser2::ResultTypeChanged(const UClass*)
{
	AllocateDefaultPins();
}

void UK2Node_EvaluateChooser2::PreloadRequiredAssets()
{
	if (Chooser)
	{
		Chooser->ConditionalPreload();
	}
    		
	Super::PreloadRequiredAssets();
}

void UK2Node_EvaluateChooser2::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	if (!FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input))
	{
		// Input - Execution Pin
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	}

	if (!FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Output))
	{
		// Output - Execution Pin
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	}

	if (Chooser)
	{
		// We show input struct contexts pins as "references", because the supplied struct might get modified if it's an In/Out param.
		// We also show output struct contexts pins as references, to show they are there just for convenience and actually pointing to the very same
		// struct that the one on the input side.
		FCreatePinParams StructContextPinCreationParams;
		StructContextPinCreationParams.bIsReference = true;
		
		for (FInstancedStruct& ContextDataEntry : Chooser->ContextData)
		{
			if (ContextDataEntry.IsValid())
			{
				const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
				if (EntryType == FContextObjectTypeClass::StaticStruct())
				{
					const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
					if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Input))
					{
						Pin->PinType.PinSubCategoryObject = ClassContext.Class;
					}
					else
					{
						CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, ClassContext.Class, ClassContext.Class.GetFName());
					}
				}
				else if (EntryType == FContextObjectTypeStruct::StaticStruct())
				{
					const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();

					if (StructContext.Direction == EContextObjectDirection::Read || StructContext.Direction == EContextObjectDirection::ReadWrite)
					{
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
						{
							Pin->PinType.PinSubCategoryObject = StructContext.Struct;
						}
						else
						{
							CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, StructContext.Struct, StructContext.Struct.GetFName(), StructContextPinCreationParams);
						}
					}

					if (StructContext.Direction == EContextObjectDirection::Write || StructContext.Direction == EContextObjectDirection::ReadWrite)
					{
						const bool bArrayOutput = StructOutputMode == EEvaluateChooserStructOutputMode::Default && Mode == EEvaluateChooserMode::AllResults;
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
						{
							Pin->PinType.PinSubCategoryObject = StructContext.Struct;
							Pin->PinType.ContainerType = bArrayOutput ? EPinContainerType::Array : EPinContainerType::None;
						}
						else
						{
							UEdGraphNode::FCreatePinParams PinParams;
							PinParams.ContainerType = bArrayOutput ? EPinContainerType::Array : EPinContainerType::None;
							PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Struct;
							
							CreatePin(EGPD_Output, PinParams.ValueTerminalType.TerminalCategory, StructContext.Struct, StructContext.Struct.GetFName(), PinParams);
						}
					}

					if (StructContext.Direction == EContextObjectDirection::Read)
					{
					    // for read only structs, remove the output pin if found
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
						{
							RemovePin(Pin);
						}
					}
					
					if (StructContext.Direction == EContextObjectDirection::Write)
					{
					    // for write only structs, remove the input pin if found
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
						{
							RemovePin(Pin);
						}
					}
				}
			}
		}
		
		UEdGraphPin* ResultPin = FindPin(TEXT("Result"), EGPD_Output);
		if (Chooser->ResultType == EObjectChooserResultType::NoPrimaryResult)
		{
			// No need to keep existing result pin if this chooser doesn't work with primary results
			if (ResultPin)
			{
				RemovePin(ResultPin);
			}
		}
		else
		{
			UClass* ResultType = UObject::StaticClass();
			bool bResultIsClass = false;
			if (Chooser->OutputObjectType)
			{
				ResultType = Chooser->OutputObjectType;
				// For class results, in Array mode, we can't easily do the CastToClass - we have to just return an Array of Objects
				bResultIsClass = Chooser->ResultType == EObjectChooserResultType::ClassResult && (Mode != EEvaluateChooserMode::AllResults);
			}

			if (ResultPin)
			{
				ResultPin->PinType.PinCategory = bReturnSoftObjectReference ? UEdGraphSchema_K2::PC_SoftObject
												: bResultIsClass ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object;

				ResultPin->PinType.PinSubCategoryObject = ResultType;
				ResultPin->PinType.ContainerType = (Mode == EEvaluateChooserMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
			}
			else
			{
				UEdGraphNode::FCreatePinParams PinParams;
				PinParams.ContainerType = (Mode == EEvaluateChooserMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
				PinParams.ValueTerminalType.TerminalCategory = bReturnSoftObjectReference ? UEdGraphSchema_K2::PC_SoftObject
												: bResultIsClass ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object;
				
				CreatePin(EGPD_Output, PinParams.ValueTerminalType.TerminalCategory, ResultType, TEXT("Result"), PinParams);
			}
		}
	}
}

FText UK2Node_EvaluateChooser2::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Chooser)
	{
		return FText::Format(LOCTEXT("EvaluateChooser2_TitleWithChooser", "Evaluate Chooser: {0}"), {FText::FromString(Chooser->GetName())});
	}
	else
	{
		return LOCTEXT("EvaluateChooser2_Title", "Evaluate Chooser");
	}
}

FText UK2Node_EvaluateChooser2::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if (Pin->PinName == UEdGraphSchema_K2::PN_Execute)
	{
		return FText();
	}
	return FText::FromName(Pin->PinName);
}

void UK2Node_EvaluateChooser2::PostPlacedNewNode()
{
	// Class default value (Single Struct output mode) is there to not break existing content.
	// for new nodes, set the default to match the object output mode
	StructOutputMode = EEvaluateChooserStructOutputMode::Default;
}

void UK2Node_EvaluateChooser2::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FString PropertyName = PropertyChangedEvent.Property->GetName();
	
	if (PropertyName == "Chooser")
	{
		ChooserChanged();
	}
	if (PropertyName == "Mode" || PropertyName == "StructOutputMode" || PropertyName == "bReturnSoftObjectReference")
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_EvaluateChooser2::PostLoad()
{
	Super::PostLoad();

	bShowModeSetting = false;
	
	if (Chooser)
	{
		Chooser->OnOutputObjectTypeChanged.AddUObject(this, &UK2Node_EvaluateChooser2::ResultTypeChanged);
		Chooser->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
		
		if (Chooser->ResultType != EObjectChooserResultType::NoPrimaryResult)
		{
			bShowModeSetting = true;
		}
		else
		{
			// NoPrimaryResult tables only set the output to their first result anyway
			Mode = EEvaluateChooserMode::FirstResult;
		}
	}

	CurrentCallbackChooser = Chooser;
}

void UK2Node_EvaluateChooser2::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_EvaluateChooser2::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
}

void UK2Node_EvaluateChooser2::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);
}

FText UK2Node_EvaluateChooser2::GetTooltipText() const
{
	return NodeTooltip;
}

void UK2Node_EvaluateChooser2::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_EvaluateChooser2::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* ExecInput = GetExecPin();
	UEdGraphPin* ExecOutput = GetPassThroughPin(ExecInput);
	UEdGraphPin* ResultPin = FindPin(TEXT("Result"));
	ensure(Chooser == nullptr || ResultPin || Chooser->ResultType == EObjectChooserResultType::NoPrimaryResult);

	UEdGraphPin* PreviousNodeExecOutput = nullptr;
	
	if (ExecInput->HasAnyConnections())
	{
		UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, this);

		if (bReturnSoftObjectReference)
		{
			if (Mode == EEvaluateChooserMode::AllResults)
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseMultiSoft)));
			}
			else
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseSoft)));
			}
		}
		else
		{
			if (Mode == EEvaluateChooserMode::AllResults)
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseMulti)));
			}
			else
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBase)));
			}
		}
		CallFunction->AllocateDefaultPins();
		
		UK2Node_CallFunction* ContextStructNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		ContextStructNode->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, MakeChooserEvaluationContext)));
        CompilerContext.MessageLog.NotifyIntermediateObjectCreation(ContextStructNode, this);
        ContextStructNode->AllocateDefaultPins();
		CompilerContext.MovePinLinksToIntermediate(*ExecInput, *ContextStructNode->GetExecPin());
		UEdGraphPin* ContextStructPin = ContextStructNode->GetReturnValuePin();
		
		ContextStructPin->MakeLinkTo(CallFunction->FindPin(FName("Context")));
		PreviousNodeExecOutput = ContextStructNode->GetThenPin();

		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(SelfNode, this);
		SelfNode->AllocateDefaultPins();
		UEdGraphPin* SelfPin = SelfNode->FindPin(TEXT("self"));

		int OutputStructArrayIndex = 0;
		
		if (Chooser)
		{
			bool bFoundObject = false;
			int ContextDataCount = Chooser->ContextData.Num();
			for(int ContextDataIndex = 0; ContextDataIndex < ContextDataCount; ContextDataIndex++)
			{
				FInstancedStruct& ContextDataEntry = Chooser->ContextData[ContextDataIndex];
				if (ContextDataEntry.IsValid())
				{
					const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
					if (EntryType == FContextObjectTypeClass::StaticStruct())
					{
						bFoundObject = true;
						const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
						if (ClassContext.Class)
						{
							if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Input))
							{
								UK2Node_CallFunction* AddObjectFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
								CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddObjectFunction, this);
								AddObjectFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserObjectInput)));
								AddObjectFunction->AllocateDefaultPins();
								ContextStructPin->MakeLinkTo(AddObjectFunction->FindPin(FName("Context")));

								PreviousNodeExecOutput->MakeLinkTo(AddObjectFunction->GetExecPin());
								PreviousNodeExecOutput = AddObjectFunction->GetThenPin();
								
								UEdGraphPin* AddObjectPin = AddObjectFunction->FindPin(FName("Object"));
								
								if (Pin->HasAnyConnections())
								{
									CompilerContext.MovePinLinksToIntermediate(*Pin, *AddObjectPin);
								}
								else // would be nice to check that self is the same type as this Object parameter
								{
									// auto connect self node to any disconnected object pin
									SelfPin->MakeLinkTo(AddObjectPin);
								}
							}
						}
					}
					else if (EntryType == FContextObjectTypeStruct::StaticStruct())
					{
						const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();
						if (StructContext.Struct)
						{
							UK2Node_CallFunction* AddStructFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
							CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddStructFunction, this);
							
							if (StructContext.Direction == EContextObjectDirection::Read || Mode == EEvaluateChooserMode::FirstResult || StructOutputMode == EEvaluateChooserStructOutputMode::SingleOutputStruct)
							{
								AddStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserStructInput)));
							}
							else
							{
								AddStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserStructInputOutput)));
							}
							
							AddStructFunction->AllocateDefaultPins();
							ContextStructPin->MakeLinkTo(AddStructFunction->FindPin(FName("Context")));

							PreviousNodeExecOutput->MakeLinkTo(AddStructFunction->GetExecPin());
							PreviousNodeExecOutput = AddStructFunction->GetThenPin();
								
							UEdGraphPin* AddStructPin = AddStructFunction->FindPin(FName("Value"));
							// not sure why this isn't automatically happening:
							AddStructPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
							AddStructPin->PinType.PinSubCategoryObject = StructContext.Struct;

							bool bLinked = false;
							if (StructContext.Direction != EContextObjectDirection::Write)
							{
								if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
								{
									if (Pin->HasAnyConnections())
									{
										CompilerContext.MovePinLinksToIntermediate(*Pin, *AddStructPin);
										bLinked = true;
									}
								}
							}
							
							if (!bLinked)
							{
								// create a struct to hold the output (or for input structs that were not connected to anything)
								UK2Node_MakeStruct* OutputStructNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeStruct>(this, SourceGraph);
								CompilerContext.MessageLog.NotifyIntermediateObjectCreation(OutputStructNode, this);
								OutputStructNode->PostPlacedNewNode();
								OutputStructNode->StructType = static_cast<UScriptStruct*>(StructContext.Struct);
								OutputStructNode->AllocateDefaultPins();
								OutputStructNode->FindPin(StructContext.Struct->GetFName(), EGPD_Output)->MakeLinkTo(AddStructPin);
							}

							if (StructContext.Direction != EContextObjectDirection::Read)
							{
								
								if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
								{
									if (Pin->HasAnyConnections())
									{
										// set up struct output pin
										UK2Node_CallFunction* GetStructFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
										CompilerContext.MessageLog.NotifyIntermediateObjectCreation(GetStructFunction, this);
										
										if (Mode == EEvaluateChooserMode::FirstResult || StructOutputMode == EEvaluateChooserStructOutputMode::SingleOutputStruct)
										{
											GetStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserStructOutput)));
											GetStructFunction->AllocateDefaultPins();
											GetStructFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(ContextDataIndex);
										}
										else
										{
											GetStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserStructOutputArray)));
											GetStructFunction->AllocateDefaultPins();
											GetStructFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(OutputStructArrayIndex);
										}
										
										ContextStructPin->MakeLinkTo(GetStructFunction->FindPin(FName("Context")));
										UEdGraphPin* ValuePin = GetStructFunction->FindPin(FName("Value"));
										// not sure why this isn't automatically happening:
										ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
										ValuePin->PinType.PinSubCategoryObject = StructContext.Struct;
						
										CompilerContext.MovePinLinksToIntermediate(*Pin, *ValuePin);
									}
								}
								OutputStructArrayIndex ++;
							}
						}
					}
				}
			}

			if (!bFoundObject)
			{
				// add Self reference to the end of the context, for debugging purposes
				UK2Node_CallFunction* AddObjectFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddObjectFunction, this);
				AddObjectFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserObjectInput)));
				AddObjectFunction->AllocateDefaultPins();
				ContextStructPin->MakeLinkTo(AddObjectFunction->FindPin(FName("Context")));
	
				PreviousNodeExecOutput->MakeLinkTo(AddObjectFunction->GetExecPin());
				PreviousNodeExecOutput = AddObjectFunction->GetThenPin();
									
				UEdGraphPin* AddObjectPin = AddObjectFunction->FindPin(FName("Object"));
				SelfPin->MakeLinkTo(AddObjectPin);
			}
			

			if (PreviousNodeExecOutput)
			{
				PreviousNodeExecOutput->MakeLinkTo(CallFunction->GetExecPin());
			}
			else
			{
				CompilerContext.MovePinLinksToIntermediate(*ExecInput, *CallFunction->GetExecPin());
			}
			
			CompilerContext.MovePinLinksToIntermediate(*ExecOutput, *CallFunction->GetThenPin());
		}

		UK2Node_CallFunction* MakeEvaluateChooserFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeEvaluateChooserFunction, this);
		MakeEvaluateChooserFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, MakeEvaluateChooser)));
		MakeEvaluateChooserFunction->AllocateDefaultPins();
		
		UEdGraphPin* ChooserTablePin = MakeEvaluateChooserFunction->FindPin(FName("Chooser"));
		CallFunction->GetSchema()->TrySetDefaultObject(*ChooserTablePin, Chooser);

		MakeEvaluateChooserFunction->GetReturnValuePin()->MakeLinkTo(CallFunction->FindPin(FName("ObjectChooser")));

		UEdGraphPin* OutputPin = CallFunction->GetReturnValuePin();

		if (Chooser) 
		{
			if (Chooser->OutputObjectType)
			{
				if (UEdGraphPin* OutputClassPin = CallFunction->FindPin(TEXT("ObjectClass")))
				{
					CallFunction->GetSchema()->TrySetDefaultObject(*OutputClassPin, Chooser->OutputObjectType);
				}
			}

			if (UEdGraphPin* ResultIsClassPin = CallFunction->FindPin(TEXT("bResultIsClass")))
			{
				// this ensures that the function does the right kind of type validation
				CallFunction->GetSchema()->TrySetDefaultValue(*ResultIsClassPin, Chooser->ResultType == EObjectChooserResultType::ClassResult ? TEXT("true") : TEXT("false"));

				// For class results, in Array mode, we can't easily do the CastToClass - we have to just return an Array of Objects
				if (Chooser->ResultType == EObjectChooserResultType::ClassResult && Mode != EEvaluateChooserMode::AllResults)
				{
					// if it's a class we need to add a CastToClass function to cast it from an Object pointer to a Class pointer
					UK2Node_CallFunction* CastToClass = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
					CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CastToClass, this);

					CastToClass->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, Conv_ObjectToClass)));
					CastToClass->AllocateDefaultPins();

					if (UEdGraphPin* ClassPin = CastToClass->FindPin(TEXT("Class")))
					{
						CastToClass->GetSchema()->TrySetDefaultObject(*ClassPin, Chooser->OutputObjectType);
					}

					if (UEdGraphPin* ObjectPin = CastToClass->FindPin(FName("Object")))
					{
						CallFunction->GetReturnValuePin()->MakeLinkTo(ObjectPin);
					}

					OutputPin = CastToClass->GetReturnValuePin();
				}
			}
		}

		if (ResultPin)
		{
			CompilerContext.MovePinLinksToIntermediate(*ResultPin, *OutputPin);
		}
	}
	
	BreakAllNodeLinks();
}

UK2Node::ERedirectType UK2Node_EvaluateChooser2::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_EvaluateChooser2::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_EvaluateChooser2::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EvaluateChooser2::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Animation);
}


#undef LOCTEXT_NAMESPACE
