// Copyright Epic Games, Inc. All Rights Reserved.


#include "GetChooserContextParametersNode.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "ChooserFunctionLibrary.h"
#include "ChooserPropertyAccess.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "IHasContext.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet/KismetSystemLibrary.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "K2Node_Self.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetChooserContextParametersNode)

#define LOCTEXT_NAMESPACE "GetChooserContextParametersNode"


//----------------------------------------------------------------------------------------------
// UK2Node_GetChooserContextParameters
// New Implementation of EvaluateChooser with support for passing in/out multiple objects and structs

UK2Node_GetChooserContextParameters::UK2Node_GetChooserContextParameters(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("GetChooserParametersNodeTooltip", "Returns all chooser parameters from a ChooserEvaluationContext struct reference.");
}

void UK2Node_GetChooserContextParameters::UnregisterCallback()
{
	if(CurrentCallbackObject)
	{
		CurrentCallbackObject->OnContextClassChanged.RemoveAll(this);
		CurrentCallbackObject = nullptr;
	}
}

void UK2Node_GetChooserContextParameters::BeginDestroy()
{
	UnregisterCallback();
	Super::BeginDestroy();
}

void UK2Node_GetChooserContextParameters::PostEditUndo()
{
	Super::PostEditUndo();
	ChooserSignatureChanged();
}

void UK2Node_GetChooserContextParameters::DestroyNode()
{
	UnregisterCallback();
	Super::DestroyNode();
}


void UK2Node_GetChooserContextParameters::ChooserSignatureChanged()
{
	if (ChooserSignature != CurrentCallbackObject)
	{
		UnregisterCallback();
		
		if (ChooserSignature)
		{
			ChooserSignature->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
		}
	
		CurrentCallbackObject = ChooserSignature;

		ReconstructNode();
	}
}

void UK2Node_GetChooserContextParameters::PreloadRequiredAssets()
{
	if (ChooserSignature)
	{
		ChooserSignature->ConditionalPreload();
	}
    		
	Super::PreloadRequiredAssets();
}

void UK2Node_GetChooserContextParameters::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	FName ContextPinName("Context");
	if (!FindPin(ContextPinName, EGPD_Input))
	{
		// Input -  Context Pin
		FCreatePinParams StructContextPinCreationParams;
		StructContextPinCreationParams.bIsReference = true;
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FChooserEvaluationContext::StaticStruct(), ContextPinName, StructContextPinCreationParams);
	}
	
	if (ChooserSignature)
	{
		FCreatePinParams StructContextPinCreationParams;
		StructContextPinCreationParams.bIsReference = true;
		
		for (FInstancedStruct& ContextDataEntry : ChooserSignature->ContextData)
		{
			if (ContextDataEntry.IsValid())
			{
				const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
				if (EntryType == FContextObjectTypeClass::StaticStruct())
				{
					const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
					if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Output))
					{
						Pin->PinType.PinSubCategoryObject = ClassContext.Class;
					}
					else
					{
						CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, ClassContext.Class, ClassContext.Class.GetFName());
					}
				}
				else if (EntryType == FContextObjectTypeStruct::StaticStruct())
				{
					const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();

					if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
					{
						Pin->PinType.PinSubCategoryObject = StructContext.Struct;
					}
					else
					{
						CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, StructContext.Struct, StructContext.Struct.GetFName(), StructContextPinCreationParams);
					}
				}
			}
		}
	}
}

FText UK2Node_GetChooserContextParameters::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("GetChooserContextParametersTitle", "Get Chooser Context Parameters");
}

FText UK2Node_GetChooserContextParameters::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if (Pin->PinName == UEdGraphSchema_K2::PN_Execute)
	{
		return FText();
	}
	return FText::FromName(Pin->PinName);
}

void UK2Node_GetChooserContextParameters::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetName() == "ChooserSignature")
	{
		ChooserSignatureChanged();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_GetChooserContextParameters::PostLoad()
{
	Super::PostLoad();
	
	if (ChooserSignature)
	{
		ChooserSignature->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
	}

	CurrentCallbackObject = ChooserSignature;
}

void UK2Node_GetChooserContextParameters::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_GetChooserContextParameters::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
}

void UK2Node_GetChooserContextParameters::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);
}

FText UK2Node_GetChooserContextParameters::GetTooltipText() const
{
	return NodeTooltip;
}

void UK2Node_GetChooserContextParameters::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_GetChooserContextParameters::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* ContextPin = FindPin(TEXT("Context"));
	check(ContextPin);
	
	if (ChooserSignature)
	{
		int ContextDataCount = ChooserSignature->ContextData.Num();
		for(int ContextDataIndex = 0; ContextDataIndex < ContextDataCount; ContextDataIndex++)
		{
				FInstancedStruct& ContextDataEntry = ChooserSignature->ContextData[ContextDataIndex];
				if (ContextDataEntry.IsValid())
				{
					const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
					if (EntryType == FContextObjectTypeClass::StaticStruct())
					{
						const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
						if (ClassContext.Class)
						{
							if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Output))
							{
								if (Pin->HasAnyConnections())
								{
									// setup class output pin
									
									UK2Node_CallFunction* GetParameterFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
									CompilerContext.MessageLog.NotifyIntermediateObjectCreation(GetParameterFunction, this);
									
									GetParameterFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserObjectInput)));
									GetParameterFunction->AllocateDefaultPins();
									GetParameterFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(ContextDataIndex);
	 									
									UEdGraphPin* ParameterContextPin = GetParameterFunction->FindPin(FName("Context"));
									CompilerContext.CopyPinLinksToIntermediate(*ContextPin, *ParameterContextPin);
									
									if (UEdGraphPin* OutputClassPin = GetParameterFunction->FindPin(TEXT("ObjectClass")))
									{
										// this makes the output pin of the function become this class type, avoiding an extra cast node
										GetParameterFunction->GetSchema()->TrySetDefaultObject(*OutputClassPin, ClassContext.Class);
									}
	 									
									UEdGraphPin* ValuePin = GetParameterFunction->GetReturnValuePin();
									
									CompilerContext.MovePinLinksToIntermediate(*Pin, *ValuePin);
								}
							}
						}
					}
	 				else if (EntryType == FContextObjectTypeStruct::StaticStruct())
	 				{
	 					const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();
	 					if (StructContext.Struct)
	 					{
	 							if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
	 							{
	 								if (Pin->HasAnyConnections())
	 								{
	 									// set up struct output pin
	 									
	 									UK2Node_CallFunction* GetStructFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	 									CompilerContext.MessageLog.NotifyIntermediateObjectCreation(GetStructFunction, this);
	 									// GetChooserStructOutput function also works for inputs (this node is outputting all members of the context input and output)
	 									GetStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserStructOutput)));
	 									GetStructFunction->AllocateDefaultPins();
	 									GetStructFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(ContextDataIndex);
	 									
										UEdGraphPin* StructContextPin = GetStructFunction->FindPin(FName("Context"));
	 									CompilerContext.CopyPinLinksToIntermediate(*ContextPin, *StructContextPin);
	 									
	 									UEdGraphPin* ValuePin = GetStructFunction->FindPin(FName("Value"));
	 									// not sure why this isn't automatically happening:
	 									ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
	 									ValuePin->PinType.PinSubCategoryObject = StructContext.Struct;
	 					
	 									CompilerContext.MovePinLinksToIntermediate(*Pin, *ValuePin);
	 								}
	 							}
	 					}
	 				}
	 			}
		}
		
	}
	
	BreakAllNodeLinks();
}

UK2Node::ERedirectType UK2Node_GetChooserContextParameters::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_GetChooserContextParameters::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_GetChooserContextParameters::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetChooserContextParameters::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Animation);
}


#undef LOCTEXT_NAMESPACE
