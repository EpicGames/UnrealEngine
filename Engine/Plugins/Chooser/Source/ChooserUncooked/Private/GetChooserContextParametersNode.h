// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "ChooserSignature.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GetChooserContextParametersNode.generated.h"


//----------------------------------------------------------------------------------------------
// UK2Node_GetChooserContextParameters
// exposes pins from a chooser context, based on a Chooser Signature
UCLASS(MinimalAPI)
class UK2Node_GetChooserContextParameters : public UK2Node
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool IsNodePure() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual void PreloadRequiredAssets() override;

	virtual void DestroyNode() override;
	//~ End UK2Node Interface.

private:
	void ChooserSignatureChanged();
	void UnregisterCallback();
	UChooserSignature* CurrentCallbackObject = nullptr;

	UPROPERTY(EditAnywhere, Category = "ChooserSignature")
	TObjectPtr<UChooserSignature> ChooserSignature;

	/** Tooltip text for this node. */
	FText NodeTooltip;
};
