// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNode_ChooserPlayer.h"
#include "AnimGraphNode_BlendStack.h"
#include "IHasContext.h"
#include "AnimGraphNode_ChooserPlayer.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_ChooserPlayer : public UAnimGraphNode_BlendStack_Base, public IHasContextClass
{
	GENERATED_BODY()

public:

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ChooserPlayer Node;

	// IHasContextClass interface
	virtual TConstArrayView<FInstancedStruct> GetContextData() const { return Node.ChooserContextDefinition; }
	// end of IHasContextClass interface
	
	// UEdGraphNode interface
	virtual void PostPlacedNewNode() override;
	virtual void PostPasteNode() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UEdGraphNode interface

	virtual FAnimNode_BlendStack_Standalone* GetBlendStackNode() const override { return (FAnimNode_BlendStack_Standalone*)(&Node); }

	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface


private:
	void UpdateContextData();
};
