// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IObjectChooser.h"
#include "ChooserPropertyAccess.h"
#include "IChooserParameterProxyTable.h"
#include "StructUtils/InstancedStructContainer.h"
#include "ProxyTable.h"
#include "LookupProxy.generated.h"

#define UE_API PROXYTABLE_API

struct FBindingChainElement;

USTRUCT()
struct FProxyTableContextProperty :  public FChooserParameterProxyTableBase
{
	GENERATED_BODY()
public:
	
	UPROPERTY(EditAnywhere, Meta = (BindingType = "UProxyTable*"), Category = "Binding")
	FChooserPropertyBinding Binding;

	UE_API virtual bool GetValue(FChooserEvaluationContext& Context, const UProxyTable*& OutResult) const override;

	CHOOSER_PARAMETER_BOILERPLATE();
};

USTRUCT(DisplayName = "Lookup Proxy", Meta = (Category = "Proxy Table", Tooltip = "Find a Proxy Asset entry in a Proxy Table, and evaluate it's value if this row is selected."))
struct FLookupProxy : public FObjectChooserBase
{
	GENERATED_BODY()
	UE_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserIteratorCallback Callback) const final override;
	UE_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;

	UE_API FLookupProxy();
	
	UE_API virtual void Compile(IHasContextClass* HasContext, bool bForce) override;
	UE_API virtual bool HasCompileErrors(FText& Message) override;

	UE_API virtual void GetDebugName(FString& OutName) const override;
	
	public:
	
	UPROPERTY(EditAnywhere, Category="Parameters")
	TObjectPtr<UProxyAsset> Proxy;

	UPROPERTY(EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/ProxyTable.ChooserParameterProxyTableBase"), Category = "Parameters")
	FInstancedStruct ProxyTable;
};

USTRUCT(meta=(Hidden))
struct FLookupProxyWithOverrideTable : public FObjectChooserBase
{
	GENERATED_BODY()
	UE_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext& ContextData, FObjectChooserSoftObjectIteratorCallback Callback) const final override;
	UE_API virtual EIteratorStatus ChooseMulti(FChooserEvaluationContext &Context, FObjectChooserIteratorCallback Callback) const final override;
	UE_API virtual UObject* ChooseObject(FChooserEvaluationContext& Context) const final override;
	UE_API virtual void ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const override;

public:
	
	UPROPERTY(EditAnywhere, Category="Parameters")
	TObjectPtr<UProxyAsset> Proxy;
	
	UPROPERTY(EditAnywhere, Category="Parameters")
	TObjectPtr<UProxyTable> OverrideProxyTable;
};

#undef UE_API
