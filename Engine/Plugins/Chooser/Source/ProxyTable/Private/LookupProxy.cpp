// Copyright Epic Games, Inc. All Rights Reserved.
#include "LookupProxy.h"
#include "ProxyTableFunctionLibrary.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ChooserPropertyAccess.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LookupProxy)

FLookupProxy::FLookupProxy()
{
	ProxyTable.InitializeAs(FProxyTableContextProperty::StaticStruct());
}

FObjectChooserBase::EIteratorStatus FLookupProxy::ChooseMulti(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	if (Proxy)
	{
		VALIDATE_CHOOSER_CONTEXT(Proxy, Proxy->ContextData, Context);

		if (const FChooserParameterProxyTableBase* ProxyTableParameter = ProxyTable.GetPtr<FChooserParameterProxyTableBase>())
		{
			const UProxyTable* Table = nullptr;
			if (ProxyTableParameter->GetValue(Context, Table))
			{
				if (Table)
				{
					return Table->FindProxyObjectMulti(Proxy->Guid, Context, Callback);
				}
			}
		}
		// fallback codepath will look up the table from the property binding on the proxy asset
		return Proxy->FindProxyObjectMulti(Context, Callback);
	}
	return FObjectChooserBase::EIteratorStatus::Failed;
}

UObject* FLookupProxy::ChooseObject(FChooserEvaluationContext& Context) const
{
	if (Proxy)
	{
		VALIDATE_CHOOSER_CONTEXT(Proxy, Proxy->ContextData, Context);
		if (const FChooserParameterProxyTableBase* ProxyTableParameter = ProxyTable.GetPtr<FChooserParameterProxyTableBase>())
		{
			const UProxyTable* Table = nullptr;
			if (ProxyTableParameter->GetValue(Context, Table))
			{
				if (Table)
				{
					return Table->FindProxyObject(Proxy->Guid, Context);
				}
			}
		}
		// fallback codepath will look up the table from the property binding on the proxy asset
		return Proxy->FindProxyObject(Context);
	}
	return nullptr;
}

UObject* FLookupProxyWithOverrideTable::ChooseObject(FChooserEvaluationContext& Context) const
{
	if (Proxy && OverrideProxyTable)
	{
		return OverrideProxyTable->FindProxyObject(Proxy->Guid, Context);
	}
	return nullptr;
}

void FLookupProxyWithOverrideTable::ChooseObject(FChooserEvaluationContext& Context, TSoftObjectPtr<>& Result) const
{
	if (Proxy && OverrideProxyTable)
   	{
   		OverrideProxyTable->FindProxyObject(Proxy->Guid, Context, Result);
   	}
}

FObjectChooserBase::EIteratorStatus FLookupProxyWithOverrideTable::ChooseMulti(FChooserEvaluationContext& Context,
                                                                               FObjectChooserSoftObjectIteratorCallback Callback) const
{
	if (Proxy && OverrideProxyTable)
   	{
   		return OverrideProxyTable->FindProxyObjectMulti(Proxy->Guid, Context, Callback);
   	}
   	return FObjectChooserBase::EIteratorStatus::Failed;
}

FObjectChooserBase::EIteratorStatus FLookupProxyWithOverrideTable::ChooseMulti(FChooserEvaluationContext& Context, FObjectChooserIteratorCallback Callback) const
{
	if (Proxy && OverrideProxyTable)
	{
		return OverrideProxyTable->FindProxyObjectMulti(Proxy->Guid, Context, Callback);
	}
	return FObjectChooserBase::EIteratorStatus::Failed;
}

bool FProxyTableContextProperty::GetValue(FChooserEvaluationContext& Context, const UProxyTable*& OutResult) const
{
	UProxyTable** ProxyTableReference;
	if (Binding.GetValuePtr(Context, ProxyTableReference))
	{
		OutResult = *ProxyTableReference;
		return true;
	}
	else
	{
		return false;
	}
}

void FLookupProxy::Compile(IHasContextClass* HasContext, bool bForce)
{
	if (Proxy)
	{
		if (FChooserParameterBase* ProxyTableParam = ProxyTable.GetMutablePtr<FChooserParameterBase>())
		{
			// todo: should also validate here that the ProxyAsset context is compatible with the passed in HasContext
			ProxyTableParam->Compile(HasContext, bForce);
		}
	}
}

bool FLookupProxy::HasCompileErrors(FText& Message)
{
	if (Proxy)
	{
		if (FChooserParameterBase* ProxyTableParam = ProxyTable.GetMutablePtr<FChooserParameterBase>())
		{
			FText BindingMessage;
			if (ProxyTableParam->HasCompileErrors(BindingMessage))
			{
				Message = FText::Format(NSLOCTEXT("LookupProxy", "LookupProxyBinding", "LookupProxy: ProxyTable Reference - {0}"), BindingMessage);
				return true;
			}
			else
			{
				return false;
			}
		}
		else
		{
			Message = NSLOCTEXT("LookupProxy", "NoParameter", "LookupProxy: No Proxy Table parameter bound");
        	return true;
		}
	}
	else
	{
		Message = NSLOCTEXT("LookupProxy", "NoProxy", "LookupProxy: Proxy Asset is not set");
		return true;
	}
}


void FLookupProxy::GetDebugName(FString& OutName) const
{
	if (Proxy)
	{
		OutName = Proxy.GetName();
	}
}
