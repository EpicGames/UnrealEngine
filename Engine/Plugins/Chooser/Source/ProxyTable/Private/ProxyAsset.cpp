// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProxyAsset.h"
#include "ProxyTableFunctionLibrary.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ChooserPropertyAccess.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "LookupProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyAsset)

FName UProxyAsset::TypeTagName = "ProxyType";

void UProxyAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	FString ResultTypeName;
	if (Type)
	{
		ResultTypeName = Type.GetName();
	}
	Context.AddTag(FAssetRegistryTag(TypeTagName, ResultTypeName, FAssetRegistryTag::TT_Alphabetical));
	
	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UProxyAsset::PostEditUndo()
{
	UObject::PostEditUndo();

	if (CachedPreviousType != Type || CachedPreviousResultType != ResultType)
	{
		OnTypeChanged.Broadcast(Type);
		CachedPreviousType = Type;
		CachedPreviousResultType = ResultType;
	}
	
	OnContextClassChanged.Broadcast();
}

void UProxyAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	
	static FName TypeName = "Type";
	static FName ResultTypeName = "ResultType";
	if (PropertyChangedEvent.Property->GetName() == TypeName)
	{
		if (CachedPreviousType != Type)
		{
			OnTypeChanged.Broadcast(Type);
		}
		CachedPreviousType = Type;
	}
	else if (PropertyChangedEvent.Property->GetName() == ResultTypeName)
	{
		if (CachedPreviousResultType != ResultType)
		{
			OnTypeChanged.Broadcast(Type);
			CachedPreviousResultType = ResultType;
		}
	}
	else
	{
		OnContextClassChanged.Broadcast();
	}
}

#endif

#if WITH_EDITORONLY_DATA

/////////////////////////////////////////////////////////////////////////////////////////
// Proxy Asset

void UProxyAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (ContextClass_DEPRECATED)
		{
			ContextData.SetNum(1);
			ContextData[0].InitializeAs<FContextObjectTypeClass>();
			FContextObjectTypeClass& Context = ContextData[0].GetMutable<FContextObjectTypeClass>();
			Context.Class = ContextClass_DEPRECATED;
			Context.Direction = EContextObjectDirection::ReadWrite;
			ContextClass_DEPRECATED = nullptr;
		}

		if (!Guid.IsValid())
		{
			// if we load a ProxyAsset that was created before the Guid, assign it a deterministic guid based on the name and path.
			Guid.A = GetTypeHash(GetName());
			Guid.B = GetTypeHash(GetPackage()->GetPathName());
		}
	}
#endif
}

void UProxyAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	CachedPreviousType = Type;
	CachedPreviousResultType = ResultType;
#endif
}

void UProxyAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	UObject::PostDuplicate(DuplicateMode);
	if (DuplicateMode == EDuplicateMode::Normal)
	{
		// create a new guid when duplicating
		Guid = FGuid::NewGuid();
	}
}

#endif

UProxyAsset::UProxyAsset(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

UObject* UProxyAsset::FindProxyObject(FChooserEvaluationContext& Context) const
{
	return nullptr;
}

FObjectChooserBase::EIteratorStatus UProxyAsset::FindProxyObjectMulti(FChooserEvaluationContext &Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const
{
	return FObjectChooserBase::EIteratorStatus::Failed;
}
