// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProxyTable.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Logging/LogMacros.h"
#include "Misc/StringBuilder.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyTable)

DEFINE_LOG_CATEGORY_STATIC(LogProxyTable,Log,All);

#if WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////////////////
/// Proxy Entry

// Gets the uint32 key that will be used as the unique identifier for the Proxy Entry
const FGuid FProxyEntry::GetGuid() const
{
	if (Proxy)
	{
		return Proxy->Guid;
	}
	else
	{
		// fallback for old FName key content
		FGuid Guid;
		if (Key != NAME_None)
		{
			Guid.A = GetTypeHash(WriteToString<128>(Key).ToView()); // Make sure this matches UProxyTableFunctionLibrary::EvaluateProxyTable
		}
		return Guid;
	}
}

// == operator for TArray::Contains
bool FProxyEntry::operator== (const FProxyEntry& Other) const
{
	return GetGuid() == Other.GetGuid();
}

// < operator for Algo::BinarySearch, and TArray::StableSort
bool FProxyEntry::operator< (const FProxyEntry& Other) const
{
	return GetGuid() < Other.GetGuid();
}

//////////////////////////////////////////////////////////////////////////////////////
/// Proxy Table

static void BuildRuntimeDataRecursive(UProxyTable* RootTable, UProxyTable* Table, TArray<FProxyEntry>& OutEntriesArray, TArray<TWeakObjectPtr<UProxyTable>>& OutDependencies)
{
	if(Table == nullptr)
	{
		return;
	}

	if (Table != RootTable)
	{
		OutDependencies.Add(Table);
	}
	
	for (const FProxyEntry& Entry : Table->Entries)
	{
		if (Entry.Proxy)
		{
			Entry.Proxy->ConditionalPostLoad();
		}

		int32 FoundIndex = OutEntriesArray.Find(Entry);
		if (FoundIndex == INDEX_NONE)
		{
			OutEntriesArray.Add(Entry);
		}
		else
		{
			// check for Guid collisions
			if (Entry.Proxy && OutEntriesArray[FoundIndex].Proxy)
			{
				if (Entry.Proxy != OutEntriesArray[FoundIndex].Proxy)
				{
					UE_LOGF(LogProxyTable, Error, "Proxy Assets %ls, and %ls have the same Guid. They may have been duplicated outside the editor.",
						*Entry.Proxy.GetName(), *OutEntriesArray[FoundIndex].Proxy.GetName());
				}
			}
			else
			{
				// fallback for FName based keys
				if (Entry.Key != OutEntriesArray[FoundIndex].Key)
				{
					UE_LOGF(LogProxyTable, Error, "Proxy Key %ls, and %ls have the same Hash.",
						*Entry.Key.ToString(), *OutEntriesArray[FoundIndex].Key.ToString());
				}
			}
		}
	}

	for (const TObjectPtr<UProxyTable>& ParentTable : Table->InheritEntriesFrom)
	{
		if (!OutDependencies.Contains(ParentTable))
		{
			BuildRuntimeDataRecursive(RootTable, ParentTable, OutEntriesArray, OutDependencies);
		}
	}
}

void UProxyTable::BuildRuntimeData()
{
	// Unregister callbacks on current dependencies
	for (TWeakObjectPtr<UProxyTable> Dependency : TableDependencies)
	{
		if (Dependency.IsValid())
		{
			Dependency->OnProxyTableChanged.RemoveAll(this);
		}
	}
	TableDependencies.Empty();
	ProxyDependencies.Empty();

	TArray<FProxyEntry> RuntimeEntries;
	BuildRuntimeDataRecursive(this, this, RuntimeEntries, TableDependencies);

	// sort by Key
	RuntimeEntries.StableSort();

	// Copy to Key and Value arrays
	int EntryCount = RuntimeEntries.Num();
	
	Keys.Empty(EntryCount);
	RuntimeValues.Empty();
	
	for(const FProxyEntry& Entry : RuntimeEntries) 
	{
		Keys.Add(Entry.GetGuid());
   		RuntimeValues.Add({Entry.Proxy, Entry.ValueStruct, Entry.OutputStructData});
	}
	
	// register callbacks on updated dependencies
	for (TWeakObjectPtr<UProxyTable> Dependency : TableDependencies)
	{
		Dependency->OnProxyTableChanged.AddUObject(this, &UProxyTable::BuildRuntimeData);
	}

	// keep a copy of proxy assets just for debugging purposes (indexes match the Key and Value arrays)
	for(const FProxyEntry& Entry : RuntimeEntries) 
	{
		if (Entry.Proxy)
		{
			ProxyDependencies.Add(Entry.Proxy);
		}
	}
}

void UProxyTable::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	UObject::PostTransacted(TransactionEvent);
	BuildRuntimeData();
	OnProxyTableChanged.Broadcast();
}

#endif

void UProxyTable::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITORONLY_DATA
	if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		BuildRuntimeData();
	}
#endif

	// compilation for property accesses
	for (FRuntimeProxyValue& Entry : RuntimeValues)
	{

		if (Entry.ProxyAsset)
		{
			if (FObjectChooserBase* Result = Entry.Value.GetMutablePtr<FObjectChooserBase>())
			{
				// need to compile results in case one is a LookupProxy
				Result->Compile(Entry.ProxyAsset, false);
			}

			/// compile any struct output property references
			for (FProxyStructOutput& StructOutput : Entry.OutputStructData)
			{
				StructOutput.Binding.Compile(Entry.ProxyAsset);
	#if WITH_EDITORONLY_DATA
				if (!StructOutput.Binding.CompileMessage.IsEmpty())
				{
					UE_LOGF(LogChooser, Error, "ProxyTable Struct Output Compile Error (%ls - %ls): %ls.", *GetName(), *Entry.ProxyAsset.GetName(), *StructOutput.Binding.CompileMessage.ToString());
				}
	#endif
			}
		}
		else
		{
			UE_LOGF(LogChooser, Display, "ProxyTable has an entry with no ProxyAsset assigned (%ls)", *GetName());
		}

	}
}


void UProxyTable::BeginDestroy()
{
	RuntimeValues.Empty();
#if WITH_EDITORONLY_DATA
	Entries.Empty();
#endif
	Super::BeginDestroy();
}

static void OutputStructData(const UProxyTable* ProxyTable, const FRuntimeProxyValue& EntryValueData, FChooserEvaluationContext& Context)
{
	for (const FProxyStructOutput& StructOutput : EntryValueData.OutputStructData)
	{
		VALIDATE_CHOOSER_CONTEXT(EntryValueData.ProxyAsset, EntryValueData.ProxyAsset->ContextData, Context);

		// copy each struct output value
		void* TargetData = nullptr;
		const UStruct* StructType = nullptr;
		if (StructOutput.Binding.GetStructPtr(Context, TargetData, StructType))
		{
			if (StructType == StructOutput.Value.GetScriptStruct())
			{
				StructOutput.Value.GetScriptStruct()->CopyScriptStruct(TargetData, StructOutput.Value.GetMemory());
			}
			else
			{
				UE_LOGF(LogChooser, Warning, "Proxy Table %ls Struct Output Type mismatch for Proxy Asset: %ls",
					ToCStr(ProxyTable->GetName()), ToCStr(EntryValueData.ProxyAsset->GetName()));
			}
		}
	}
}

UObject* UProxyTable::FindProxyObject(const FGuid& Key, FChooserEvaluationContext& Context) const
{
	const int FoundIndex = Algo::BinarySearch(Keys, Key);
	if (FoundIndex != INDEX_NONE)
	{
		const FRuntimeProxyValue& EntryValueData = RuntimeValues[FoundIndex];

		OutputStructData(this, EntryValueData, Context);

		if (const FObjectChooserBase *EntryValue = EntryValueData.Value.GetPtr<const FObjectChooserBase>())
		{
			return EntryValue->ChooseObject(Context);
		}

		return nullptr;
	}
	
	return nullptr;
}

void UProxyTable::FindProxyObject(const FGuid& Key, FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const
{
	const int FoundIndex = Algo::BinarySearch(Keys, Key);
	if (FoundIndex != INDEX_NONE)
	{
		const FRuntimeProxyValue& EntryValueData = RuntimeValues[FoundIndex];
		
		OutputStructData(this, EntryValueData, Context);

		if (const FObjectChooserBase *EntryValue = EntryValueData.Value.GetPtr<const FObjectChooserBase>())
		{
			EntryValue->ChooseObject(Context, Result);
		}
	}
}

FObjectChooserBase::EIteratorStatus UProxyTable::FindProxyObjectMulti(const FGuid& Key, FChooserEvaluationContext &Context, FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback Callback) const
{
	const int FoundIndex = Algo::BinarySearch(Keys, Key);
    if (FoundIndex != INDEX_NONE)
    {
    	const FRuntimeProxyValue& EntryValueData = RuntimeValues[FoundIndex];
    	
    	OutputStructData(this, EntryValueData, Context);
    	
    	if (const FObjectChooserBase *EntryValue = EntryValueData.Value.GetPtr<const FObjectChooserBase>())
    	{
    		return EntryValue->ChooseMulti(Context, Callback);
    	}
    
    	return FObjectChooserBase::EIteratorStatus::Failed;
    }
    	
    return FObjectChooserBase::EIteratorStatus::Failed;
}

FObjectChooserBase::EIteratorStatus UProxyTable::FindProxyObjectMulti(const FGuid& Key, FChooserEvaluationContext &Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const
{
	const int FoundIndex = Algo::BinarySearch(Keys, Key);
    if (FoundIndex != INDEX_NONE)
    {
    	const FRuntimeProxyValue& EntryValueData = RuntimeValues[FoundIndex];
    	
		OutputStructData(this, EntryValueData, Context);
    	
    	if (const FObjectChooserBase *EntryValue = EntryValueData.Value.GetPtr<const FObjectChooserBase>())
    	{
    		return EntryValue->ChooseMulti(Context, Callback);
    	}
    
    	return FObjectChooserBase::EIteratorStatus::Failed;
    }
    	
    return FObjectChooserBase::EIteratorStatus::Failed;
}

FObjectChooserBase::EIteratorStatus UProxyTable::IterateProxyObjects(const FGuid& Key, FChooserEvaluationContext& Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const
{
	const int FoundIndex = Algo::BinarySearch(Keys, Key);
	if (FoundIndex != INDEX_NONE)
	{
		const FRuntimeProxyValue& EntryValueData = RuntimeValues[FoundIndex];
		const FObjectChooserBase& EntryValue = EntryValueData.Value.Get<const FObjectChooserBase>();

		FObjectChooserBase::EIteratorStatus Result = EntryValue.IterateObjects(Context, Callback);
		return Result;
	}

	return FObjectChooserBase::EIteratorStatus::Failed;
}

FObjectChooserBase::EIteratorStatus UProxyTable::IterateProxyObjects(const FGuid& Key, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const
{
	FChooserEvaluationContext Context;
	return IterateProxyObjects(Key, Context, Callback);
}

UProxyTable::UProxyTable(const FObjectInitializer& Initializer)
	:Super(Initializer)
{

}
