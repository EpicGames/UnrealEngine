// Copyright Epic Games, Inc. All Rights Reserved.
#include "ProxyTableFunctionLibrary.h"

#include "LookupProxy.h"
#include "Misc/StringBuilder.h"
#include "ProxyTable.h"

/////////////////////////////////////////////////////////////////////////////////////
// Blueprint Library Functions

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyTableFunctionLibrary)

UObject* UProxyTableFunctionLibrary::EvaluateProxyAsset(const UObject* ContextObject, const UProxyAsset* Proxy, TSubclassOf<UObject> ObjectClass)
{
	UObject* Result = nullptr;
	if (Proxy)
	{
		FChooserEvaluationContext Context(const_cast<UObject*>(ContextObject));
		
		Result = Proxy->FindProxyObject(Context);
		if (ObjectClass && Result && !Result->IsA(ObjectClass)) //-V522
		{
			return nullptr;
		}
	}
	return Result;
	
}

// fallback for FName based Keys:
UObject* UProxyTableFunctionLibrary::EvaluateProxyTable(const UObject* ContextObject, const UProxyTable* ProxyTable, FName Key)
{
	if (ProxyTable)
	{
		FGuid Guid;
		Guid.A = GetTypeHash(WriteToString<128>(Key).ToView()); // Make sure this matches FProxyEntry::GetGuid
		FChooserEvaluationContext Context(const_cast<UObject*>(ContextObject));
		if (UObject* Value = ProxyTable->FindProxyObject(Guid, Context))
		{
			return Value;
		}
	}
	
	return nullptr;
}


FInstancedStruct UProxyTableFunctionLibrary::MakeLookupProxy(UProxyAsset* Proxy)
{
 	FInstancedStruct Struct;
 	Struct.InitializeAs(FLookupProxy::StaticStruct());
 	Struct.GetMutable<FLookupProxy>().Proxy = Proxy;
 	return Struct;
}

FInstancedStruct UProxyTableFunctionLibrary::MakeLookupProxyWithOverrideTable(UProxyAsset* Proxy, UProxyTable* ProxyTable)
{
 	FInstancedStruct Struct;
 	Struct.InitializeAs(FLookupProxyWithOverrideTable::StaticStruct());
 	FLookupProxyWithOverrideTable& Data = Struct.GetMutable<FLookupProxyWithOverrideTable>();
	Data.Proxy = Proxy;
	Data.OverrideProxyTable = ProxyTable;
 	return Struct;
}

#if WITH_EDITOR
TMap<UProxyAsset*, UObject*> UProxyTableFunctionLibrary::GetProxyTableEntries(const UProxyTable* ProxyTable)
{
	TMap<UProxyAsset*, UObject*> Result;
	if (ProxyTable == nullptr)
	{
		return Result;
	}

	const int32 NumEntries = FMath::Min(ProxyTable->Keys.Num(), ProxyTable->RuntimeValues.Num());
	for (int32 Index = 0; Index < NumEntries; Index++)
	{
		const FRuntimeProxyValue& RuntimeValue = ProxyTable->RuntimeValues[Index];
		UProxyAsset* ProxyAsset = RuntimeValue.ProxyAsset.Get();
		if (ProxyAsset == nullptr)
		{
			continue;
		}

		UObject* Value = nullptr;
		if (const FObjectChooserBase* Chooser = RuntimeValue.Value.GetPtr<FObjectChooserBase>())
		{
			Value = Chooser->GetReferencedObject();
		}

		Result.Add(ProxyAsset, Value);
	}

	return Result;
}
#endif

