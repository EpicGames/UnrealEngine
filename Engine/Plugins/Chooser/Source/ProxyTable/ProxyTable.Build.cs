// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProxyTable : ModuleRules
	{
		public ProxyTable(ReadOnlyTargetRules Target) : base(Target)
		{			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Chooser"
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}