// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "IChooserParameterBase.h"
#include "StructUtils/InstancedStruct.h"
#include "IObjectChooser.h"
#include "IChooserParameterProxyTable.generated.h"

class UProxyTable;

UINTERFACE(MinimalAPI, NotBlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UChooserParameterProxyTable : public UInterface
{
	GENERATED_BODY()
};

class IChooserParameterProxyTable
{
	GENERATED_BODY()
public:
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) const {}
};


USTRUCT()
struct FChooserParameterProxyTableBase : public FChooserParameterBase
{
	GENERATED_BODY()
    
public:
	virtual bool GetValue(FChooserEvaluationContext& Context, const UProxyTable*& OutResult) const { return false; }
};
