// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IHasContext.h"
#include "StructUtils/InstancedStruct.h"
#include "Misc/Guid.h"
#include "IObjectChooser.h"
#include "ProxyAsset.generated.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FProxyTypeChanged, const UClass* OutputObjectType);

/**
* EXPERIMENTAL: An indirect mapping to an asset. The direct mapping is defined in a Proxy Table.
*/
UCLASS(MinimalAPI,BlueprintType,Experimental)
class UProxyAsset : public UObject, public IHasContextClass
{
	GENERATED_UCLASS_BODY()
public:
	UProxyAsset() {}

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	static PROXYTABLE_API FName TypeTagName;

#if WITH_EDITOR
	FProxyTypeChanged OnTypeChanged;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// caching the type so that on Undo, we can tell if we should fire the changed delegate
	UClass* CachedPreviousType = nullptr;
	EObjectChooserResultType CachedPreviousResultType = EObjectChooserResultType::ObjectResult;
#endif
	

	UPROPERTY(EditAnywhere, Category = "Proxy", Meta = (AllowAbstract=true))
	TObjectPtr<UClass> Type;
	UPROPERTY(EditAnywhere, Category = "Proxy")
	EObjectChooserResultType ResultType = EObjectChooserResultType::ObjectResult;
	
	UPROPERTY(EditAnywhere, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ContextObjectTypeBase"), Category = "Input")
	TArray<FInstancedStruct> ContextData;

	UPROPERTY()
	FGuid Guid;

	virtual TConstArrayView<FInstancedStruct> GetContextData() const override { return ContextData; }
	virtual FString GetContextOwnerName() const override { return GetName(); }
	virtual UObject* GetContextOwnerAsset() override { return this; }

	UObject* FindProxyObject(struct FChooserEvaluationContext& Context) const;
	FObjectChooserBase::EIteratorStatus FindProxyObjectMulti(FChooserEvaluationContext &Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UClass> ContextClass_DEPRECATED;
	
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#endif
};