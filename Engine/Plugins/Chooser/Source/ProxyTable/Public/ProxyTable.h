// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "ChooserPropertyAccess.h"
#include "StructUtils/InstancedStruct.h"
#include "Misc/Guid.h"
#include "ProxyAsset.h"
#include "ProxyTable.generated.h"

#define UE_API PROXYTABLE_API

USTRUCT()
struct FProxyStructOutput
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Data")
	FChooserStructPropertyBinding Binding;
	
	UPROPERTY(EditAnywhere, NoClear, Category = "Data")
	FInstancedStruct Value;
};

USTRUCT()
struct FProxyEntry
{
	GENERATED_BODY()

	UE_API bool operator== (const FProxyEntry& Other) const;
	UE_API bool operator< (const FProxyEntry& Other) const;
	
	UPROPERTY(EditAnywhere, Category = "Data")
	TObjectPtr<UProxyAsset> Proxy;

	// temporarily leaving this property for backwards compatibility with old content which used FNames rather than UProxyAsset
	UPROPERTY(EditAnywhere, Category = "Data")
	FName Key;
	
	UPROPERTY(DisplayName="Value", EditAnywhere, NoClear, Meta = (ExcludeBaseStruct, BaseStruct = "/Script/Chooser.ObjectChooserBase"), Category = "Data")
	FInstancedStruct ValueStruct;

	UPROPERTY(EditAnywhere, NoClear, Category = "Data")
	TArray<FProxyStructOutput> OutputStructData;

	UE_API const FGuid GetGuid() const;
};

#if WITH_EDITORONLY_DATA
inline uint32 GetTypeHash(const FProxyEntry& Entry) { return GetTypeHash(Entry.GetGuid()); }
#endif

DECLARE_MULTICAST_DELEGATE(FProxyTableChanged);

USTRUCT()
struct FRuntimeProxyValue
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UProxyAsset> ProxyAsset;

	UPROPERTY()
	FInstancedStruct Value;

	UPROPERTY()
	TArray<FProxyStructOutput> OutputStructData;
};

/**
* EXPERIMENTAL: Table mapping of proxy assets to a specific asset.
*/
UCLASS(MinimalAPI, BlueprintType, Experimental)
class UProxyTable : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	UProxyTable() {}
	UE_API virtual void BeginDestroy() override;

	UPROPERTY()
	TArray<FGuid> Keys;

	UPROPERTY()
	TArray<FRuntimeProxyValue> RuntimeValues;

	UE_API FObjectChooserBase::EIteratorStatus FindProxyObjectMulti(const FGuid& Key, FChooserEvaluationContext &Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const;
	UE_API UObject* FindProxyObject(const FGuid& Key, FChooserEvaluationContext& Context) const;
	void FindProxyObject(const FGuid& Key, FChooserEvaluationContext& Context, TSoftObjectPtr<UObject>& Result) const;
	FObjectChooserBase::EIteratorStatus FindProxyObjectMulti(const FGuid& Key, FChooserEvaluationContext& Context,
	                                                         FObjectChooserBase::FObjectChooserSoftObjectIteratorCallback Callback) const;
	UE_API FObjectChooserBase::EIteratorStatus IterateProxyObjects(const FGuid& Key, FChooserEvaluationContext& Context, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const;
	UE_API FObjectChooserBase::EIteratorStatus IterateProxyObjects(const FGuid& Key, FObjectChooserBase::FObjectChooserIteratorCallback Callback) const;

	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
public:
	FProxyTableChanged OnProxyTableChanged;
	
	UPROPERTY(EditAnywhere, Category = "Hidden")
	TArray<FProxyEntry> Entries;
	
	UPROPERTY(EditAnywhere, Category = "Inheritance")
	TArray<TObjectPtr<UProxyTable>> InheritEntriesFrom;

	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
private:
	UE_API void BuildRuntimeData();
	TArray<TWeakObjectPtr<UProxyTable>> TableDependencies;
	TArray<TWeakObjectPtr<UProxyAsset>> ProxyDependencies;
#endif

};

#undef UE_API
