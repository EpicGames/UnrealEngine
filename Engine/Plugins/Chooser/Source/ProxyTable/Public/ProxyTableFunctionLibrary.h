// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "ProxyTable.h"
#include "Templates/SubclassOf.h"
#include "ProxyTableFunctionLibrary.generated.h"

#define UE_API PROXYTABLE_API

/**
 * Proxy Table Function Library
 */
UCLASS(MinimalAPI)
class UProxyTableFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
		* Temporary backwards compatibility function!  please switch to EvaluateProxyAsset
		*
		* @param ContextObject			(in) An Object from which the parameters to the Chooser Table will be read
		* @param ProxyTable				(in) The ProxyTable asset
		* @param Key					(in) The Key from the ProxyTable asset
	*/
	UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe, BlueprintInternalUseOnly="true"), Category = "Animation")
	static UE_API UObject* EvaluateProxyTable(const UObject* ContextObject, const UProxyTable* ProxyTable, FName Key);
	
	/**
	* Resolve a proxy asset and return the selected UObject, or null
	*
	* @param ContextObject			(in) An Object from which the Proxy Table will be read, and parameters to any nested Chooser Tables that need to evaluate
	* @param ProxyAsset				(in) The ProxyAsset asset
	* @param ObjectClass			(in) Expected type of result objects
	*/
	UFUNCTION(BlueprintPure, meta = (BlueprintThreadSafe, DeterminesOutputType = "ObjectClass", BlueprintInternalUseOnly="true"), Category = "Animation")
	static UE_API UObject* EvaluateProxyAsset(const UObject* ContextObject, const UProxyAsset* Proxy, TSubclassOf<UObject> ObjectClass);
	
	/**
	* Create a LookupProxy struct
	*
	* @param Chooser				(in) the ChooserTable asset to evaluate
	*/
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static UE_API FInstancedStruct MakeLookupProxy(UProxyAsset* Proxy);
	
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe, BlueprintInternalUseOnly="true"))
	static UE_API FInstancedStruct MakeLookupProxyWithOverrideTable(UProxyAsset* Proxy, UProxyTable* ProxyTable);

#if WITH_EDITOR
	/**
	* Get all entries in a ProxyTable as a map from ProxyAsset to its resolved UObject value.
	* Editor-only: uses editor-only APIs (GetReferencedObject) that are not available in cooked builds.
	*
	* @param ProxyTable				(in) The ProxyTable asset to read entries from
	* @return Map of ProxyAsset to resolved UObject value
	*/
	UFUNCTION(BlueprintCallable, Category = "Animation", meta = (CallInEditor = "true"))
	static UE_API TMap<UProxyAsset*, UObject*> GetProxyTableEntries(const UProxyTable* ProxyTable);
#endif
};

#undef UE_API
