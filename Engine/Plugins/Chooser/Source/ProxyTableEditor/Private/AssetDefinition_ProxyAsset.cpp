// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ProxyAsset.h"

#include "ProxyTableEditor.h"
#include "ProxyTableEditorStyle.h"
#include "Toolkits/SimpleAssetEditor.h"

#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ProxyAsset)

#define LOCTEXT_NAMESPACE "UAssetDefinition_ProxyAsset" 

EAssetCommandResult UAssetDefinition_ProxyAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	FSimpleAssetEditor::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Objects);

	return EAssetCommandResult::Handled;
}

const FSlateBrush* UAssetDefinition_ProxyAsset::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	FString ProxyType;
	if (InAssetData.GetTagValue<FString>(UProxyAsset::TypeTagName, ProxyType))
	{
		if (ProxyType.Contains("AnimSequence"))
		{
			return UE::ProxyTableEditor::FProxyTableEditorStyle::Get().GetBrush("ProxyTableEditor.ProxyAnimSequenceIconLarge");
		}
	}
	
	return UE::ProxyTableEditor::FProxyTableEditorStyle::Get().GetBrush("ProxyTableEditor.ProxyAssetIconLarge");
}

const FSlateBrush* UAssetDefinition_ProxyAsset::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ProxyTableEditor::FProxyTableEditorStyle::Get().GetBrush("ProxyTableEditor.ProxyAssetIconLarge");
}

#undef LOCTEXT_NAMESPACE
