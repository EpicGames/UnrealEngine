// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ProxyTable.h"

#include <ProxyTableEditorStyle.h>

#include "ProxyTableEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ProxyTable)

#define LOCTEXT_NAMESPACE "UAssetDefinition_ProxyTable" 

EAssetCommandResult UAssetDefinition_ProxyTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	
	for (UObject* Object : Objects)
	{
		UE::ProxyTableEditor::FProxyTableEditor::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}
const FSlateBrush* UAssetDefinition_ProxyTable::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ProxyTableEditor::FProxyTableEditorStyle::Get().GetBrush("ProxyTableEditor.ProxyTableIconLarge");
}

const FSlateBrush* UAssetDefinition_ProxyTable::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return UE::ProxyTableEditor::FProxyTableEditorStyle::Get().GetBrush("ProxyTableEditor.ProxyTableIconLarge");
}

#undef LOCTEXT_NAMESPACE
