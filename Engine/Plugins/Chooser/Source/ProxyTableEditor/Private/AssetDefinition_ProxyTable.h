// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProxyTable.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ProxyTable.generated.h"

UCLASS()
class UAssetDefinition_ProxyTable : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ProxyTable", "Proxy Table"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,128,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UProxyTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Misc) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	
	const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	// UAssetDefinition End
};
