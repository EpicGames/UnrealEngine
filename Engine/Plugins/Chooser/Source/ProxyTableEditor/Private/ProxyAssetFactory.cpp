// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyAssetFactory.h"
#include "ProxyTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyAssetFactory)

UProxyAssetFactory::UProxyAssetFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UProxyAsset::StaticClass();
}

bool UProxyAssetFactory::ConfigureProperties()
{
	return true;
}

UObject* UProxyAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	UProxyAsset* Asset = NewObject<UProxyAsset>(InParent, Class, Name, Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted);
	Asset->Guid = FGuid::NewGuid();
	return Asset;
}
