// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "ProxyAssetFactory.generated.h"

UCLASS(MinimalAPI)
class UProxyAssetFactory : public UFactory
{
	GENERATED_BODY()

	UProxyAssetFactory();
	
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override
	{
		return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
	}

	virtual FName GetNewAssetThumbnailOverride() const override { return "ProxyTableEditor.ProxyAssetIconLarge"; }
};