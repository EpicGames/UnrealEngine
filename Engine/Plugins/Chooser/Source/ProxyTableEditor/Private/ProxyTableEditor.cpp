// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyTableEditor.h"

#include "AssetViewUtils.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "SAssetDropTarget.h"
#include "SClassViewer.h"
#include "SourceCodeNavigation.h"
#include "ProxyTable.h"
#include "ClassViewerFilter.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyAccessEditor.h"
#include "LandscapeRender.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "GraphEditorSettings.h"
#include "IDetailCustomization.h"
#include "ObjectChooserClassFilter.h"
#include "ScopedTransaction.h"
#include "ObjectChooserWidgetFactories.h"
#include "IObjectChooser.h"
#include "Widgets/Layout/SSeparator.h"
#include "PropertyCustomizationHelpers.h"
#include "ProxyTableEditorCommands.h"
#include "LookupProxy.h"
#include "SPropertyAccessChainWidget.h"
#include "Widgets/Layout/SScrollBox.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyTableEditor)

#define LOCTEXT_NAMESPACE "ProxyTableEditor"

namespace UE::ProxyTableEditor
{

const FName FProxyTableEditor::ToolkitFName( TEXT( "GenericAssetEditor" ) );
const FName FProxyTableEditor::PropertiesTabId( TEXT( "ProxyEditor_Properties" ) );
const FName FProxyTableEditor::TableTabId( TEXT( "ProxyEditor_Table" ) );
	

class FProxyRowDetails : public IDetailCustomization
{
public:
	FProxyRowDetails() {};
	virtual ~FProxyRowDetails() override {};

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable( new FProxyRowDetails() );
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
};

// Make the details panel show the values for the selected row, showing each column value
void FProxyRowDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	UProxyRowDetails* Row = Cast<UProxyRowDetails>(Objects[0]);
	UProxyTable* ProxyTable = Row->ProxyTable;
	
	if (ProxyTable->Entries.IsValidIndex(Row->Row))
	{
		IDetailCategoryBuilder& PropertiesCategory = DetailBuilder.EditCategory("Row Properties");

		TSharedPtr<IPropertyHandle> ProxyTableProperty = DetailBuilder.GetProperty("ProxyTable", Row->StaticClass());
		DetailBuilder.HideProperty(ProxyTableProperty);
	
		TSharedPtr<IPropertyHandle> EntriesArrayProperty = ProxyTableProperty->GetChildHandle("Entries");
		TSharedPtr<IPropertyHandle> CurrentEntryProperty = EntriesArrayProperty->AsArray()->GetElement(Row->Row);
		IDetailPropertyRow& NewEntryProperty = PropertiesCategory.AddProperty(CurrentEntryProperty);
		NewEntryProperty.DisplayName(LOCTEXT("Entry","Selected Entry"));
		NewEntryProperty.ShowPropertyButtons(false); // hide array add button
		NewEntryProperty.ShouldAutoExpand(true);
	}
}
	
void FProxyTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_GenericAssetEditor", "Asset Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( PropertiesTabId, FOnSpawnTab::CreateSP(this, &FProxyTableEditor::SpawnPropertiesTab) )
		.SetDisplayName( LOCTEXT("PropertiesTab", "Details") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon("EditorStyle", "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner( TableTabId, FOnSpawnTab::CreateSP(this, &FProxyTableEditor::SpawnTableTab) )
		.SetDisplayName( LOCTEXT("TableTab", "Proxy Table") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon("EditorStyle", "LevelEditor.Tabs.Details"));
}

void FProxyTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( TableTabId );
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
}

const FName FProxyTableEditor::ProxyEditorAppIdentifier( TEXT( "ProxyEditorApp" ) );

FProxyTableEditor::~FProxyTableEditor()
{
	ClearSelectedRows();
	
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	
	DetailsView.Reset();
}

void FProxyTableEditor::RegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* ToolBar;
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	if (ToolMenus->IsMenuRegistered(MenuName))
	{
		ToolBar = ToolMenus->ExtendMenu(MenuName);
	}
	else
	{
		ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);
	}

	const FProxyTableEditorCommands& Commands = FProxyTableEditorCommands::Get();
	FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
	{
		FToolMenuSection& Section = ToolBar->AddSection("Proxy Table", TAttribute<FText>(), InsertAfterAssetSection);
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.EditTableSettings,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon("EditorStyle", "FullBlueprintEditor.EditGlobalOptions")));
	}

}

void FProxyTableEditor::BindCommands()
{
	const FProxyTableEditorCommands& Commands = FProxyTableEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.EditTableSettings,
		FExecuteAction::CreateSP(this, &FProxyTableEditor::SelectRootProperties));
}

void FProxyTableEditor::InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	EditingObjects = ObjectsToEdit;
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FProxyTableEditor::OnObjectsReplaced);
	FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &FProxyTableEditor::OnObjectTransacted);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_ProxyTableEditor_Layout_v1" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab( TableTabId, ETabState::OpenedTab )
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab( PropertiesTabId, ETabState::OpenedTab )
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FProxyTableEditor::ProxyEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

	BindCommands();
	RegenerateMenusAndToolbars();
	RegisterToolbar();

	SelectRootProperties();
}

FName FProxyTableEditor::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FProxyTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Proxy Table Editor");
}

void FProxyTableEditor::PostUndo(bool bSuccess)
{
	UpdateTableRows();
}
	
void FProxyTableEditor::PostRedo(bool bSuccess)
{
	UpdateTableRows();
}

	
void FProxyTableEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
}

void FProxyTableEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// for all details panel changes, just refresh the table
	UpdateTableRows();
}

	
FText FProxyTableEditor::GetToolkitName() const
{
	const auto& EditingObjs = GetEditingObjects();

	check( EditingObjs.Num() > 0 );

	FFormatNamedArguments Args;
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );

	if( EditingObjs.Num() == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];
		return FText::FromString(EditingObject->GetName());
	}
	else
	{
		UClass* SharedBaseClass = nullptr;
		for( int32 x = 0; x < EditingObjs.Num(); ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == nullptr)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == nullptr )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		check(SharedBaseClass);

		Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitTitle_EditingMultiple", "{NumberOfObjects} {ClassName} - {ToolkitName}"), Args );
	}
}

FText FProxyTableEditor::GetToolkitToolTipText() const
{
	const auto& EditingObjs = GetEditingObjects();

	check( EditingObjs.Num() > 0 );

	FFormatNamedArguments Args;
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );

	if( EditingObjs.Num() == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];
		return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
	}
	else
	{
		UClass* SharedBaseClass = NULL;
		for( int32 x = 0; x < EditingObjs.Num(); ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == nullptr)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == nullptr )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		check(SharedBaseClass);

		Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitTitle_EditingMultipleToolTip", "{NumberOfObjects} {ClassName} - {ToolkitName}"), Args );
	}
}

FLinearColor FProxyTableEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.0f, 0.0f, 0.5f );
}

void FProxyTableEditor::SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate)
{
	DetailsView->SetIsPropertyVisibleDelegate(InVisibilityDelegate);
	DetailsView->ForceRefresh();
}

void FProxyTableEditor::SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate)
{
	DetailsView->SetIsPropertyEditingEnabledDelegate(InPropertyEditingDelegate);
	DetailsView->ForceRefresh();
}

TSharedRef<SDockTab> FProxyTableEditor::SpawnPropertiesTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == PropertiesTabId );

	return SNew(SDockTab)
		.Label( LOCTEXT("GenericDetailsTitle", "Details") )
		.TabColorScale( GetTabColorScale() )
		.OnCanCloseTab_Lambda([]() { return false; })
		[
			DetailsView.ToSharedRef()
		];
}


class FProxyRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWidgetTemplateDragDropOp, FDecoratedDragDropOp)

	TSharedPtr<FProxyTableEditor::FProxyTableRow> Row;

	/** Constructs the drag drop operation */
	static TSharedRef<FProxyRowDragDropOp> New(TSharedPtr<FProxyTableEditor::FProxyTableRow> InRow)
	{
		TSharedRef<FProxyRowDragDropOp> Operation = MakeShareable(new FProxyRowDragDropOp());
		Operation->Row = InRow;
		Operation->DefaultHoverText = LOCTEXT("Proxy Row", "Proxy Row");
		Operation->CurrentHoverText = Operation->DefaultHoverText;
			
		Operation->Construct();
	
		return Operation;
	};
};

class SProxyRowHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SProxyRowHandle)
	{}
	SLATE_ARGUMENT(FProxyTableEditor*, Editor)
	SLATE_ARGUMENT(TSharedPtr<FProxyTableEditor::FProxyTableRow>, Row)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ProxyEditor = InArgs._Editor;
		Row = InArgs._Row;

		ChildSlot
		[
			SNew(SBox) .Padding(0.0f) .HAlign(HAlign_Center) .VAlign(VAlign_Center) .WidthOverride(16.0f)
			[
				SNew(SImage)
				.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
			]
		];
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		ProxyEditor->SelectRow(Row);
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// clear row selection so that delete key can't cause the selected row to be deleted
		ProxyEditor->ClearSelectedRows();
		
		TSharedRef<FProxyRowDragDropOp> DragDropOp = FProxyRowDragDropOp::New(Row);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

private:
	FProxyTableEditor* ProxyEditor = nullptr;
	TSharedPtr<FProxyTableEditor::FProxyTableRow> Row;
};


class SProxyTableRow : public SMultiColumnTableRow<TSharedPtr<FProxyTableEditor::FProxyTableRow>>
{
public:
	SLATE_BEGIN_ARGS(SProxyTableRow) {}
		/** The list item for this row */
		SLATE_ARGUMENT(TSharedPtr<FProxyTableEditor::FProxyTableRow>, Entry)
		SLATE_ARGUMENT(FProxyTableEditor*, Editor)
	SLATE_END_ARGS()

	static constexpr int SpecialIndex_AddRow = -1;
	static constexpr int SpecialIndex_InheritedFrom = -2;

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		Row = Args._Entry;
		Editor = Args._Editor;

		SMultiColumnTableRow<TSharedPtr<FProxyTableEditor::FProxyTableRow>>::Construct(
			FSuperRowType::FArguments(),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		static FName Handles = "Handles";
		static FName Key = "Key";
		static FName OldKey = "OldKey";
		static FName Value = "Value";

		if (Row->RowIndex >=0)
		{
			if (ColumnName == Handles)
			{
				// row drag handle
				return SNew(SProxyRowHandle).Row(Row).Editor(Editor);
			}
			else if (ColumnName == Value) 
			{
				UClass* ObjectType = nullptr;
				if (Row->ProxyTable->Entries[Row->RowIndex].Proxy)
				{
					ObjectType = Row->ProxyTable->Entries[Row->RowIndex].Proxy->Type;
				}
				bool bReadOnly = Row->ProxyTable != Editor->GetProxyTable();
				TSharedPtr<SWidget> ResultWidget = ChooserEditor::FObjectChooserWidgetFactories::CreateWidget( bReadOnly,
					Row->ProxyTable, FObjectChooserBase::StaticStruct(),
					Row->ProxyTable->Entries[Row->RowIndex].ValueStruct.GetMutableMemory(),
					Row->ProxyTable->Entries[Row->RowIndex].ValueStruct.GetScriptStruct(),
					ObjectType,
					FOnStructPicked::CreateLambda([this](const UScriptStruct* ChosenStruct)
					{
						{
							const FScopedTransaction Transaction(LOCTEXT("Change Value Type", "Change Value Type"));
							Row->ProxyTable->Modify(true);
							Row->ProxyTable->Entries[Row->RowIndex].ValueStruct.InitializeAs(ChosenStruct);
						}
						ChooserEditor::FObjectChooserWidgetFactories::CreateWidget(false, Row->ProxyTable, FObjectChooserBase::StaticStruct(),
								Row->ProxyTable->Entries[Row->RowIndex].ValueStruct.GetMutableMemory(),
								Row->ProxyTable->Entries[Row->RowIndex].ValueStruct.GetScriptStruct(),
								Row->ProxyTable->Entries[Row->RowIndex].Proxy ? Row->ProxyTable->Entries[Row->RowIndex].Proxy->Type.Get() : UObject::StaticClass(),
								FOnStructPicked(), &CacheBorder);
					}),
					&CacheBorder
					);

				return SNew(SOverlay)
					+ SOverlay::Slot()
					[
						ResultWidget.ToSharedRef()
					]
					+ SOverlay::Slot().VAlign(VAlign_Bottom)
					[
						SNew(SSeparator).SeparatorImage(FCoreStyle::Get().GetBrush("FocusRectangle"))
						.Visibility_Lambda([this]() { return bDragActive && !bDropAbove ? EVisibility::Visible : EVisibility::Hidden; })
					]
					+ SOverlay::Slot().VAlign(VAlign_Top)
					[
						SNew(SSeparator).SeparatorImage(FCoreStyle::Get().GetBrush("FocusRectangle"))
						.Visibility_Lambda([this]() { return bDragActive && bDropAbove ? EVisibility::Visible : EVisibility::Hidden; })
					];
			}
			else if (ColumnName == OldKey)
			{
				bool bReadOnly = Row->ProxyTable != Editor->GetProxyTable();
				return SNew(SEditableTextBox)
					.IsEnabled(!bReadOnly)
					.Text_Lambda([this](){ return Row->ProxyTable->Entries.Num() > Row->RowIndex ?  FText::FromName(Row->ProxyTable->Entries[Row->RowIndex].Key) : FText();})
					.OnTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
					{
						if (Row->ProxyTable->Entries.Num() > Row->RowIndex)
						{
							Row->ProxyTable->Entries[Row->RowIndex].Key = FName(Text.ToString());
						}
					});	
			}
			else if (ColumnName == Key)
			{
				bool bReadOnly = Row->ProxyTable != Editor->GetProxyTable();
				return SNew(SObjectPropertyEntryBox).IsEnabled(!bReadOnly)
					.AllowedClass(UProxyAsset::StaticClass())
					.ObjectPath_Lambda([this]()
					{
						return Row->ProxyTable->Entries.Num() > Row->RowIndex ?  Row->ProxyTable->Entries[Row->RowIndex].Proxy.GetPath() : FString();
					})
					.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
					{
						if (Row->ProxyTable->Entries.Num() > Row->RowIndex)
						{
							const FScopedTransaction Transaction(LOCTEXT("Edit Proxy Asset", "Edit Proxy Asset"));
							Row->ProxyTable->Modify(true);
							Row->ProxyTable->Entries[Row->RowIndex].Proxy = Cast<UProxyAsset>(AssetData.GetAsset());
							
							// ideally just need to rebuild the widget for the "Value" to update the UObject type filtering.
							// For now just trigger a full refresh
							Editor->UpdateTableRows();
						}
					});
			}
		}
		else
		{
			// special case row for "Add Row" button
			if (Row->RowIndex == SpecialIndex_AddRow)
			{
				// on the row past the end, show an Add button in the result column
				if (ColumnName == Value)
				{
					return SNew(SOverlay)
						+ SOverlay::Slot()
						[
							Editor->GetCreateRowComboButton().ToSharedRef()
						]
						+ SOverlay::Slot().VAlign(VAlign_Top)
						[
							SNew(SSeparator).SeparatorImage(FCoreStyle::Get().GetBrush("FocusRectangle"))
							.Visibility_Lambda([this]() { return bDragActive ? EVisibility::Visible : EVisibility::Hidden; })
						];
				}
			}
		}
		return SNullWidget::NullWidget;
	}

	
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		if (TSharedPtr<FProxyRowDragDropOp> Operation = DragDropEvent.GetOperationAs<FProxyRowDragDropOp>())
		{
			if (Row->ProxyTable == Editor->GetProxyTable())
			{
				bDragActive = true;
				float Center = MyGeometry.Position.Y + MyGeometry.Size.Y;
				bDropAbove = DragDropEvent.GetScreenSpacePosition().Y < Center;
			}
		}
	}
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		bDragActive = false;
	}
	
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		if (Row->ProxyTable == Editor->GetProxyTable())
		{
			if (TSharedPtr<FProxyRowDragDropOp> Operation = DragDropEvent.GetOperationAs<FProxyRowDragDropOp>())
			{
				float Center = MyGeometry.AbsolutePosition.Y + MyGeometry.Size.Y/2;
				bDropAbove = DragDropEvent.GetScreenSpacePosition().Y < Center;
				return FReply::Handled();
			}
		}
		return FReply::Unhandled();
	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		if (TSharedPtr<FProxyRowDragDropOp> Operation = DragDropEvent.GetOperationAs<FProxyRowDragDropOp>())
		{
			if (Row->ProxyTable == Editor->GetProxyTable()) // only allow dropping on rows that are part of this actual table (not inherited entries)
			{
				int InsertIndex = Row->RowIndex;
				if (InsertIndex < 0)
				{
					InsertIndex = Editor->GetProxyTable()->Entries.Num();
				}
				else
				{
					if (!bDropAbove)
					{
						InsertIndex++;
					}
				}
				
				if (Row->ProxyTable == Operation->Row->ProxyTable)
				{
					// move row within a proxy table
					int NewIndex = Editor->MoveRow(Operation->Row->RowIndex, InsertIndex);
					Editor->SelectRow(NewIndex);
					return FReply::Handled();		
				}
				else
				{
					Editor->InsertEntry(Operation->Row->ProxyTable->Entries[Operation->Row->RowIndex], InsertIndex);
					Editor->SelectRow(InsertIndex);
					return FReply::Handled();
				}
			}
		}
		return FReply::Unhandled();
	}	

private:
	TSharedPtr<FProxyTableEditor::FProxyTableRow> Row;
	FProxyTableEditor* Editor;
	TSharedPtr<SBorder> CacheBorder;
	bool bDragActive = false;
	bool bDropAbove = false;	
};


TSharedRef<ITableRow> FProxyTableEditor::GenerateTableRow(TSharedPtr<FProxyTableRow> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);

	if (InItem->RowIndex == SProxyTableRow::SpecialIndex_InheritedFrom)
	{
		return SNew(STableRow<TSharedRef<FProxyTableRow>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(STextBlock)
					.Text(InItem->Children.Num() > 0 ?  LOCTEXT("Inherited from ", "Inherited from ") : LOCTEXT("No rows inherited from ", "No rows inherited from ") )
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SHyperlink)
					.Text(FText::FromString(InItem->ProxyTable->GetName()))
					.OnNavigate_Lambda([InItem]()
					{
						AssetViewUtils::OpenEditorForAsset(InItem->ProxyTable);
					})
			]
		];
	}
	else
	{
		return SNew(SProxyTableRow, OwnerTable)

			.Entry(InItem).Editor(this);
	}
}

void FProxyTableEditor::SelectRootProperties()
{
	if( DetailsView.IsValid() )
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObjects( EditingObjects );
	}
}

void FProxyTableEditor::DeleteSelectedRows()
{
	UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);
	
	const FScopedTransaction Transaction(LOCTEXT("Delete Row Transaction", "Delete Row"));
	ProxyTable->Modify(true);
	
	// delete selected rows.
	TArray<uint32> RowsToDelete;
	for(auto& SelectedRow:SelectedRows)
	{
		if (ProxyTable->Entries.IsValidIndex(SelectedRow->Row))
		{
			RowsToDelete.Add(SelectedRow->Row);
		}
	}
	// sort indices in reverse
	RowsToDelete.Sort([](int32 A, int32 B){ return A>B; });
	for(uint32 RowIndex : RowsToDelete)
	{
		ProxyTable->Entries.RemoveAt(RowIndex);
	}

	ClearSelectedRows();
	UpdateTableRows();
}
	
	
void FProxyTableEditor::ClearSelectedRows() 
{
	for(UObject* SelectedRow : SelectedRows)
	{
		SelectedRow->RemoveFromRoot();
	}
	SelectedRows.SetNum(0);
	TableView->ClearSelection();
	SelectRootProperties();
}
	
void FProxyTableEditor::SelectRow(TSharedPtr<FProxyTableRow> Row)
{
	if (!TableView->IsItemSelected(Row))
	{
		TableView->ClearSelection();
		TableView->SetItemSelection(Row, true, ESelectInfo::OnMouseClick);
	}
}

void FProxyTableEditor::InsertEntry(FProxyEntry& Entry, int RowIndex)
{
	UProxyTable* Table = Cast<UProxyTable>(EditingObjects[0]);
	RowIndex = FMath::Min(RowIndex,Table->Entries.Num());
	
	const FScopedTransaction Transaction(LOCTEXT("Move Row", "Move Row"));
	
	Table->Modify(true);

	RowIndex = FMath::Clamp(RowIndex, 0, Table->Entries.Num());

	Table->Entries.Insert(Entry, RowIndex);

	UpdateTableRows();
}

int FProxyTableEditor::MoveRow(int SourceRowIndex, int TargetRowIndex)
{
	UProxyTable* Table = Cast<UProxyTable>(EditingObjects[0]);
	TargetRowIndex = FMath::Min(TargetRowIndex,Table->Entries.Num());
	
	const FScopedTransaction Transaction(LOCTEXT("Move Row", "Move Row"));
	
	Table->Modify(true);

	FProxyEntry Entry = Table->Entries[SourceRowIndex];
	Table->Entries.RemoveAt(SourceRowIndex);
	if (SourceRowIndex < TargetRowIndex)
	{
		TargetRowIndex--;
	}
	Table->Entries.Insert(Entry, TargetRowIndex);

	UpdateTableRows();
	return TargetRowIndex;
}

void FProxyTableEditor::TreeViewExpansionChanged(TSharedPtr<FProxyTableEditor::FProxyTableRow> InItem, bool bShouldBeExpanded)
{
	if (InItem->RowIndex == SProxyTableRow::SpecialIndex_InheritedFrom)
	{
		ImportedTablesExpansionState.Add(InItem->ProxyTable, bShouldBeExpanded);
	}
}


void FProxyTableEditor::UpdateTableColumns()
{
	HeaderRow->ClearColumns();
	HeaderRow->AddColumn(SHeaderRow::Column("Handles")
					.DefaultLabel(FText())
					.ManualWidth(30));
	
	HeaderRow->AddColumn(SHeaderRow::Column("Key")
					.DefaultLabel(LOCTEXT("KeyColumnName", "Proxy"))
					.ManualWidth(500));

  // Code for allowing editing of deprecated Key data
  //   if (UProxyTable* Table = GetProxyTable())
  //   	{
  //   		if (Table->Entries.Num() > 0)
  //   		{
  //   			// if the first entry has a non-none key, assume this is an old table and make an extra column with the old FName Key property
  //   			if (Table->Entries[0].Key != NAME_None)
  //   			{
  //   					HeaderRow->AddColumn(SHeaderRow::Column("OldKey")
  //                   					.DefaultLabel(LOCTEXT("OldKeyColumnName", "Key (Deprecated)"))
  //                   					.ManualWidth(500));
  //   			}
  //   		}
  //   	}

	HeaderRow->AddColumn(SHeaderRow::Column("Value")
					.DefaultLabel(LOCTEXT("ValueColumnName", "Value"))
					.ManualWidth(500));
}

TSharedRef<SDockTab> FProxyTableEditor::SpawnTableTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == TableTabId );

	UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);
	
	CreateRowComboButton = SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.ButtonContent()
		[
			SNew(STextBlock).Text(LOCTEXT("AddRow", "+ Add Row"))
		]
		.OnGetMenuContent_Lambda([this]()
		{
			FStructViewerInitializationOptions Options;
			Options.StructFilter = MakeShared<ChooserEditor::FStructFilter>(FObjectChooserBase::StaticStruct());
			Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
			
			TSharedRef<SWidget> Widget = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, FOnStructPicked::CreateLambda([this](const UScriptStruct* ChosenStruct)
			{
				CreateRowComboButton->SetIsOpen(false);
				UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);
				const FScopedTransaction Transaction(LOCTEXT("Add Row Transaction", "Add Row"));
				ProxyTable->Modify(true);

				ProxyTable->Entries.SetNum(ProxyTable->Entries.Num()+1);
				ProxyTable->Entries.Last().ValueStruct.InitializeAs(ChosenStruct);
				UpdateTableRows();
			}));
			
			return Widget;
		});

	HeaderRow = SNew(SHeaderRow);
	
	TableView = SNew(STreeView<TSharedPtr<FProxyTableRow>>)
    			.TreeItemsSource(&TableRows)
			.OnExpansionChanged(this, &FProxyTableEditor::TreeViewExpansionChanged)
				.OnGetChildren_Lambda( [] (TSharedPtr<FProxyTableRow> Row, TArray<TSharedPtr<FProxyTableRow>>& OutChildren)
				{
					OutChildren = Row->Children;
				})
				.OnKeyDownHandler_Lambda([this](const FGeometry&, const FKeyEvent& Event)
				{	
					if ( Event.GetKey() == EKeys::Delete)
					{
						UProxyTable* ProxyTable = GetProxyTable();
						DeleteSelectedRows();
						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<FProxyTableRow> SelectedItem,  ESelectInfo::Type SelectInfo)
				{
					if (SelectedItem)
					{
						for (UObject* SelectedRow : SelectedRows)
                     	{
                     		SelectedRow->RemoveFromRoot();
                     	}
						SelectedRows.SetNum(0);
						UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);
						// Get the list of objects to edit the details of
						TObjectPtr<UProxyRowDetails> Selection = NewObject<UProxyRowDetails>();
						Selection->ProxyTable = ProxyTable;
						Selection->Row = SelectedItem->RowIndex;
						Selection->AddToRoot();
						SelectedRows.Add(Selection);
						
						TArray<UObject*> DetailsObjects;
						for(auto& Item : SelectedRows)
						{
							DetailsObjects.Add(Item.Get());
						}

						if( DetailsView.IsValid() )
						{
							// Make sure details window is pointing to our object
							DetailsView->SetObjects( DetailsObjects );
						}		
					}
				})
    			.OnGenerateRow_Raw(this, &FProxyTableEditor::GenerateTableRow)
				.HeaderRow(HeaderRow);
				

	UpdateTableColumns();
	UpdateTableRows();
	
	return SNew(SDockTab)
		.Label( LOCTEXT("ProxtTableTitle", "Proxy Table") )
		.TabColorScale( GetTabColorScale() )
		.OnCanCloseTab_Lambda([]() { return false; })
		[
			SNew(SScrollBox).Orientation(Orient_Horizontal)
			+ SScrollBox::Slot()
			[
				TableView.ToSharedRef()
			]
		];
}

void FProxyTableEditor::AddInheritedRows(UProxyTable* ProxyTable)
{
	if (ProxyTable == nullptr || ReferencedProxyTables.Find(ProxyTable))
	{
		return;
	}

	// prevent infinite inheritance loops
	ReferencedProxyTables.Add(ProxyTable);
	
	// add "Inherited from" header above inherited rows
	TSharedPtr<FProxyTableRow> ParentTableRow = MakeShared<FProxyTableRow>(SProxyTableRow::SpecialIndex_InheritedFrom, ProxyTable);
	TableRows.Add(ParentTableRow);

	bool* CachedExpansionState = ImportedTablesExpansionState.Find(ProxyTable);
	bool Expansion = CachedExpansionState ? *CachedExpansionState : true;
	TableView->SetItemExpansion(ParentTableRow, Expansion);

	int NumRows = TableRows.Num();
	for(int i =0; i<ProxyTable->Entries.Num(); i++)
	{
		// check if there's already an entry in TableRows for the same ProxyAsset
		if (!ReferencedProxyEntries.Find(ProxyTable->Entries[i]))
		{
			ParentTableRow->Children.Add(MakeShared<FProxyTableRow>(i, ProxyTable));
			ReferencedProxyEntries.Add(ProxyTable->Entries[i]);
		}
	}

	// recursively add inherited entries
	for(UProxyTable* InheritedTable : ProxyTable->InheritEntriesFrom)
	{
		AddInheritedRows(InheritedTable);
	}
}

void FProxyTableEditor::UpdateTableRows()
{
	UProxyTable* ProxyTable = Cast<UProxyTable>(EditingObjects[0]);

	TableRows.SetNum(0);
	ReferencedProxyEntries.Empty(ReferencedProxyEntries.Num());
	ReferencedProxyTables.Empty(ReferencedProxyTables.Num());

	// add rows from this table
	for(int i =0; i<ProxyTable->Entries.Num(); i++)
	{
		TableRows.Add(MakeShared<FProxyTableRow>(i, ProxyTable));
		ReferencedProxyEntries.Add(ProxyTable->Entries[i]);
	}

	// Add 1 at the end, for the "Add Row" control
	TableRows.Add(MakeShared<FProxyTableRow>(SProxyTableRow::SpecialIndex_AddRow, ProxyTable));
	
	if (ProxyTable->InheritEntriesFrom.Num()>0)
	{
		// add imported rows in to the table
		for(UProxyTable* InheritedTable : ProxyTable->InheritEntriesFrom)
		{
			AddInheritedRows(InheritedTable);
		}
	}

	if (TableView.IsValid())
	{
		TableView->RebuildList();
	}
}

void FProxyTableEditor::OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionObjectEvent)
{
	if (UProxyTable* ModifiedProxyTable = Cast<UProxyTable>(InObject))
	{
		if (ReferencedProxyTables.Contains(ModifiedProxyTable))
		{
			UpdateTableRows();
		}
	}	
}

void FProxyTableEditor::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bChangedAny = false;

	// Refresh our details view if one of the objects replaced was in the map. This gets called before the reinstance GC fixup, so we might as well fixup EditingObjects now too
	for (int32 i = 0; i < EditingObjects.Num(); i++)
	{
		UObject* SourceObject = EditingObjects[i];
		UObject* ReplacedObject = ReplacementMap.FindRef(SourceObject);

		if (ReplacedObject && ReplacedObject != SourceObject)
		{
			EditingObjects[i] = ReplacedObject;
			bChangedAny = true;
		}
	}

	if (bChangedAny)
	{
		DetailsView->SetObjects(EditingObjects);
	}
}

FString FProxyTableEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Generic Asset ").ToString();
}
	
TSharedRef<FProxyTableEditor> FProxyTableEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FProxyTableEditor > NewEditor( new FProxyTableEditor() );

	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add( ObjectToEdit );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );

	return NewEditor;
}

TSharedRef<FProxyTableEditor> FProxyTableEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FProxyTableEditor > NewEditor( new FProxyTableEditor() );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );
	return NewEditor;
}

/// Result widgets
///
	
TSharedRef<SWidget> CreateProxyTablePropertyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, UE::ChooserEditor::FChooserWidgetValueChanged ValueChanged)
{
	IHasContextClass* HasContextClass = Cast<IHasContextClass>(TransactionObject);

	FProxyTableContextProperty* ContextProperty = reinterpret_cast<FProxyTableContextProperty*>(Value);

	return SNew(UE::ChooserEditor::SPropertyAccessChainWidget).ContextClassOwner(HasContextClass).AllowFunctions(false).BindingColor("ClassPinTypeColor").TypeFilter("UProxyTable*")
	.PropertyBindingValue(&ContextProperty->Binding)
	.OnValueChanged(ValueChanged);
}

TSharedRef<SWidget> CreateLookupProxyWidget(bool bReadOnly, UObject* TransactionObject, void* Value, UClass* ResultBaseClass, ChooserEditor::FChooserWidgetValueChanged ValueChanged)
{
	FLookupProxy* LookupProxy = static_cast<FLookupProxy*>(Value);
	
	TSharedPtr<SWidget> ProxyTableWidget = UE::ChooserEditor::FObjectChooserWidgetFactories::CreateWidget(false, TransactionObject, LookupProxy->ProxyTable.GetMutableMemory(),LookupProxy->ProxyTable.GetScriptStruct(), ResultBaseClass);

	
	TSharedRef<SWidget> ProxyAssetWidget =SNew(SObjectPropertyEntryBox)
    		.IsEnabled(!bReadOnly)
    		.AllowedClass(UProxyAsset::StaticClass())
			.DisplayBrowse(false)
			.DisplayUseSelected(false)
    		.ObjectPath_Lambda([LookupProxy](){ return LookupProxy->Proxy.GetPath();})
    		.OnShouldFilterAsset_Lambda([ResultBaseClass](const FAssetData& AssetData)
    		{
    			if (ResultBaseClass == nullptr)
    			{
    				return false;
    			}
    			if (AssetData.IsInstanceOf(UProxyAsset::StaticClass()))
    			{
    				if (UProxyAsset* Proxy = Cast<UProxyAsset>(AssetData.GetAsset()))
    				{
    					return !(Proxy->Type && Proxy->Type->IsChildOf(ResultBaseClass));
    				}
    			}
    			return true;
    		})
    		.OnObjectChanged_Lambda([TransactionObject, LookupProxy, ValueChanged](const FAssetData& AssetData)
    		{
    			const FScopedTransaction Transaction(LOCTEXT("Edit Chooser", "Edit Chooser"));
    			TransactionObject->Modify(true);
    			LookupProxy->Proxy = Cast<UProxyAsset>(AssetData.GetAsset());
    			ValueChanged.ExecuteIfBound();
    		});

	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			ProxyTableWidget ? ProxyTableWidget.ToSharedRef() : SNullWidget::NullWidget
		]
		+SHorizontalBox::Slot()
		[
			ProxyAssetWidget
		];
}

void FProxyTableEditor::RegisterWidgets()
{
	UE::ChooserEditor::FObjectChooserWidgetFactories::RegisterWidgetCreator(FLookupProxy::StaticStruct(), CreateLookupProxyWidget);
	UE::ChooserEditor::FObjectChooserWidgetFactories::RegisterWidgetCreator(FProxyTableContextProperty::StaticStruct(), CreateProxyTablePropertyWidget);
	
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("ProxyRowDetails", FOnGetDetailCustomizationInstance::CreateStatic(&FProxyRowDetails::MakeInstance));	
}
	
}

#undef LOCTEXT_NAMESPACE
