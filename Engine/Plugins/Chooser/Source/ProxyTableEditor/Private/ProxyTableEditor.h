// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Editor/PropertyEditor/Public/PropertyEditorDelegates.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableViewBase.h"
#include "ProxyTable.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "ProxyTableEditor.generated.h"

class SComboButton;
class SEditableText;
class IDetailsView;

// Class used for chooser editor details customization
UCLASS()
class UProxyRowDetails : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Instanced, Category="Hidden")
	TObjectPtr<UProxyTable> ProxyTable;
	int Row;
};

namespace UE::ProxyTableEditor
{
	class FProxyTableEditor : public FAssetEditorToolkit, public FSelfRegisteringEditorUndoClient, public FNotifyHook
	{
	public:
		/** Delegate that, given an array of assets, returns an array of objects to use in the details view of an FSimpleAssetEditor */
		DECLARE_DELEGATE_RetVal_OneParam(TArray<UObject*>, FGetDetailsViewObjects, const TArray<UObject*>&);

		virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;


		/**
		* Edits the specified asset object
		*
		* @param	Mode					Asset editing mode for this editor (standalone or world-centric)
		* @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
		* @param	ObjectsToEdit			The object to edit
		* @param	GetDetailsViewObjects	If bound, a delegate to get the array of objects to use in the details view; uses ObjectsToEdit if not bound
		*/
		void InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects );

		/** Destructor */
		virtual ~FProxyTableEditor();

		/** IToolkit interface */
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitName() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual bool IsPrimaryEditor() const override { return true; }
		virtual bool IsSimpleAssetEditor() const override { return false; }
		
		/** FEditorUndoClient Interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;
		
		/** Begin FNotifyHook Interface */
		virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;
		virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

		UProxyTable* GetProxyTable() { return Cast<UProxyTable>(EditingObjects[0]); }
	
		/** Used to show or hide certain properties */
		void SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate);
		/** Can be used to disable the details view making it read-only */
		void SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate);
	
		struct FProxyTableRow
		{
			FProxyTableRow(int32 InIndex, UProxyTable* InProxyTable) :
				ProxyTable(InProxyTable),
				RowIndex(InIndex)
			{
			}
			
			UProxyTable* ProxyTable;
			int32 RowIndex;
			TArray<TSharedPtr<FProxyTableRow>> Children;			
		};

		void UpdateTableColumns();
		void UpdateTableRows();
		int MoveRow(int SourceRowIndex, int TargetIndex);
		void InsertEntry(FProxyEntry& Entry, int RowIndex);
		void DeleteSelectedRows();
		void ClearSelectedRows();
		void SelectRow(int RowIndex) { if (TableRows.IsValidIndex(RowIndex)) { SelectRow(TableRows[RowIndex]); } }
		void SelectRow(TSharedPtr<FProxyTableRow> Row);
	private:
		void AddInheritedRows(UProxyTable* ProxyTable);
		void SelectRootProperties();
		void RegisterToolbar();
		void BindCommands();
	
		/** Create the properties tab and its content */
		TSharedRef<SDockTab> SpawnPropertiesTab( const FSpawnTabArgs& Args );
		/** Create the table tab and its content */
		TSharedRef<SDockTab> SpawnTableTab( const FSpawnTabArgs& Args );
	
		TSharedRef<ITableRow> GenerateTableRow(TSharedPtr<FProxyTableRow> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		void TreeViewExpansionChanged(TSharedPtr<FProxyTableEditor::FProxyTableRow> InItem, bool bShouldBeExpanded);

		/** Called when objects need to be swapped out for new versions, like after a blueprint recompile. */
		void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
		void OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionObjectEvent);

		/** Details view */
		TSharedPtr< class IDetailsView > DetailsView;

		/** App Identifier. */
		static const FName ProxyEditorAppIdentifier;

		/**	The tab ids for all the tabs used */
		static const FName PropertiesTabId;
		static const FName TableTabId;

		/** The objects open within this editor */
		TArray<UObject*> EditingObjects;

		TArray<TObjectPtr<UProxyRowDetails>> SelectedRows;

		TArray<TSharedPtr<FProxyTableRow>> MainTableRows;
		TArray<TSharedPtr<FProxyTableRow>> TableRows;
	
		TSharedPtr<SComboButton> CreateRowComboButton;
		
		TSharedPtr<SHeaderRow> HeaderRow;
		TSharedPtr<STreeView<TSharedPtr<FProxyTableRow>>> TableView;
		
	public:
		TMap<UProxyTable*, bool> ImportedTablesExpansionState;
		TSet<FProxyEntry> ReferencedProxyEntries;
		TSet<UProxyTable*> ReferencedProxyTables;

		TSharedPtr<SComboButton>& GetCreateRowComboButton() { return CreateRowComboButton; };

		/** The name given to all instances of this type of editor */
		static const FName ToolkitFName;

		static TSharedRef<FProxyTableEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );

		static TSharedRef<FProxyTableEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );

		static void RegisterWidgets();
	};
}
