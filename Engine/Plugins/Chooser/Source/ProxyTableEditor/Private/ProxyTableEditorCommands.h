// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ProxyTableEditorCommands"
	
class FProxyTableEditorCommands : public TCommands<FProxyTableEditorCommands>
{
public:
	/** Constructor */
	FProxyTableEditorCommands() 
		: TCommands<FProxyTableEditorCommands>("ProxyTableEditor", NSLOCTEXT("Contexts", "ProxyTableEditor", "Proxy Table Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	TSharedPtr<FUICommandInfo> EditTableSettings;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(EditTableSettings, "Table Settings", "Edit the root properties of the ChooserTable asset.", EUserInterfaceActionType::Button, FInputChord())
	}
};
	
#undef LOCTEXT_NAMESPACE
