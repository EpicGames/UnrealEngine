// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyTableEditorModule.h"
#include "ProxyTableEditor.h"
#include "ProxyTableEditorCommands.h"
#include "StructOutputDataCustomization.h"
#include "PropertyEditorModule.h"
#include "ProxyTableEditorStyle.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "ProxyTableEditorModule"

namespace UE::ProxyTableEditor
{

void FModule::StartupModule()
{
	FProxyTableEditorStyle::Initialize();
	FProxyTableEditor::RegisterWidgets();

	FProxyTableEditorCommands::Register();
	
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FProxyStructOutput::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FStructOutputDataCustomization>(); }));
}

void FModule::ShutdownModule()
{
	FProxyTableEditorCommands::Unregister();

	if (UObjectInitialized() && !IsEngineExitRequested())
	{

		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(FProxyStructOutput::StaticStruct()->GetFName());
		}
	}

	FProxyTableEditorStyle::Shutdown();
}

}

IMPLEMENT_MODULE(UE::ProxyTableEditor::FModule, ProxyTableEditor);

#undef LOCTEXT_NAMESPACE
