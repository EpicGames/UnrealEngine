// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyTableEditorStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::ProxyTableEditor
{
	
TSharedPtr< FProxyTableEditorStyle > FProxyTableEditorStyle::StyleInstance = nullptr;


FProxyTableEditorStyle::FProxyTableEditorStyle() :
	FSlateStyleSet("ProxyTableEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Chooser/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT(""));

	// proxy asset icon
	Set("ProxyTableEditor.ProxyAssetIconLarge", new IMAGE_BRUSH_SVG("ProxyAsset_64", Icon24x24));
	// proxy sequence icon
	Set("ProxyTableEditor.ProxyAnimSequenceIconLarge", new IMAGE_BRUSH_SVG("ProxyAnimSequence_64", Icon24x24));

	// proxy table icon
	Set("ProxyTableEditor.ProxyTableIconLarge", new IMAGE_BRUSH_SVG("ProxyTable_64", Icon24x24));
}

void FProxyTableEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = MakeShared<FProxyTableEditorStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FProxyTableEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

const ISlateStyle& FProxyTableEditorStyle::Get()
{
	return *StyleInstance;
}

}