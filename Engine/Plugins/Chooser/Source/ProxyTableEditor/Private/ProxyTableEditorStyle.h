// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

namespace UE::ProxyTableEditor
{
	
class FProxyTableEditorStyle : FSlateStyleSet
{
public:
    FProxyTableEditorStyle();
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();

private:

	static TSharedPtr< FProxyTableEditorStyle > StyleInstance;
};

}
