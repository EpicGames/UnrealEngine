// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyTableFactory.h"
#include "ProxyTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyTableFactory)

UProxyTableFactory::UProxyTableFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UProxyTable::StaticClass();
}

bool UProxyTableFactory::ConfigureProperties()
{
	return true;
}

UObject* UProxyTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	return NewObject<UProxyTable>(InParent, Class, Name, Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted);
}
