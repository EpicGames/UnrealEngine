// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructOutputDataCustomization.h"

#include "AnimNode_ChooserPlayer.h"
#include "Chooser.h"
#include "ChooserPropertyAccess.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "GraphEditorSettings.h"
#include "SClassViewer.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyAccessEditor.h"
#include "ScopedTransaction.h"
#include "SPropertyAccessChainWidget.h"
#include "InstancedStructDetails.h"
#include "ProxyTable.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "StructOutputCustomization"

namespace UE::ProxyTableEditor
{
	
void FStructOutputDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FProperty* Property = PropertyHandle->GetProperty();

	UClass* ContextClass = nullptr;

	static FName ProxyPropertyName = "Proxy";

	
	TSharedPtr<IPropertyUtilities> PropUtils = CustomizationUtils.GetPropertyUtilities();
         	 					
	IHasContextClass* HasContext = nullptr;
	// this details customization is hard coded to work in a ProxyTable where the StructOutputs array will be the parent
	if (TSharedPtr<IPropertyHandle> ArrayHandle = PropertyHandle->GetParentHandle())
	{
		// and then the parent of that is the FProxyEntry
		if (TSharedPtr<IPropertyHandle> ProxyEntryHandle = ArrayHandle->GetParentHandle())
		{
			// and the FProxyEntry contains a UProxyAsset reference
			if (TSharedPtr<IPropertyHandle> ProxyHandle = ProxyEntryHandle->GetChildHandle(ProxyPropertyName))
			{
				if (FProperty* ProxyProperty = ProxyHandle->GetProperty())
				{
					if (FObjectProperty* ProxyObjectProperty = CastField<FObjectProperty>(ProxyProperty))
					{
						void* ValuePtr = nullptr;
						ProxyHandle->GetValueData(ValuePtr);
						const TObjectPtr<UProxyAsset>& ProxyAssetReference = *reinterpret_cast<TObjectPtr<UProxyAsset>*>(ValuePtr);

						if (ProxyAssetReference)
						{
							// and the UProxyAsset has has a context on it that we will use for the binding
							HasContext = static_cast<IHasContextClass*>(ProxyAssetReference->GetInterfaceAddress(UHasContextClass::StaticClass()));
						}
					}
				}
			}
		}
	}

	static const FName BindingName = "Binding";
	TSharedPtr<IPropertyHandle> BindingHandle = PropertyHandle->GetChildHandle(BindingName);
	
	static const FName ValueName = "Value";
	TSharedPtr<IPropertyHandle> ValueHandle = PropertyHandle->GetChildHandle(ValueName);
	
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(UE::ChooserEditor::SPropertyAccessChainWidget).ContextClassOwner(HasContext).TypeFilter("struct").BindingColor("StructPinTypeColor").AllowFunctions(false)
		.PropertyBindingValue_Lambda([BindingHandle]()
		{
			void* data;
			BindingHandle->GetValueData(data);
			return reinterpret_cast<FChooserPropertyBinding*>(data);
		})
		.OnAddBinding_Lambda([HasContext, ValueHandle, BindingHandle, PropUtils](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
         	 	{
         	 		TArray<UObject*> OuterObjects;
         	 		BindingHandle->GetOuterObjects(OuterObjects);
         	 		const FScopedTransaction Transaction(LOCTEXT("Change Property Binding", "Change Property Binding"));
         	 	
         			for (uint32 i=0; i<BindingHandle->GetNumOuterObjects(); i++)
         	 		{
         				// Set the binding
         	 			void* ValuePtr = nullptr;
         	 			BindingHandle->GetValueData(ValuePtr);
         	 			FChooserPropertyBinding* PropertyValue = reinterpret_cast<FChooserPropertyBinding*>(ValuePtr);
         	 		
         	 			if (PropertyValue != nullptr)
         	 			{
         	 				BindingHandle->NotifyPreChange();
         	 				
         	 				OuterObjects[i]->Modify(true);
         	 				Chooser::CopyPropertyChain(InBindingChain, *PropertyValue);
         	 				PropertyValue->Compile(HasContext);
         	
							FChooserStructPropertyBinding* StructPropertyBinding = static_cast<FChooserStructPropertyBinding*>(PropertyValue);

         	 				if (InBindingChain.Num() > 1)
         	 				{
         	 					FField* Property = InBindingChain.Last().Field.ToField();
								if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
								{
									StructPropertyBinding->StructType = StructProperty->Struct;
									StructPropertyBinding->DisplayName = StructProperty->GetDisplayNameText().ToString();
								}
         	 				}
         	 				else if (InBindingChain.Num() ==1)
         	 				{
         	 					// directly bound to context struct
         	 					int Index = InBindingChain[0].ArrayIndex;
								TConstArrayView<FInstancedStruct> ContextData = HasContext->GetContextData();
								if (const FContextObjectTypeStruct* StructContextData = ContextData[Index].GetPtr<FContextObjectTypeStruct>())
								{
									StructPropertyBinding->StructType = StructContextData->Struct;
									StructPropertyBinding->DisplayName = StructContextData->Struct->GetAuthoredName();
								}
         	 				}
         	 				
							// reset the type of the FInstancedStruct storing the value
							ValueHandle->GetValueData(ValuePtr);
							FInstancedStruct* ValueStruct = reinterpret_cast<FInstancedStruct*>(ValuePtr);

         	 				// begin Temporary data conversion codepath
         	 				if (StructPropertyBinding->StructType == FChooserPlayerSettings::StaticStruct() &&
								(ValueStruct->IsValid() && ValueStruct->GetScriptStruct()->GetName() == "ProxyPropPoseParams"))
         	 				{
         	 					const UScriptStruct* Struct = ValueStruct->GetScriptStruct();
         	 					const void* StructData = ValueStruct->GetMemory();

         	 					FDoubleProperty* FrameProperty = CastField<FDoubleProperty>(Struct->CustomFindProperty("Frame"));
         	 					double Frame = 0.0;
         	 					FrameProperty->GetValue_InContainer(StructData, &Frame);
         	 							
         	 					FBoolProperty* IsMirroredProperty = CastField<FBoolProperty>(Struct->CustomFindProperty("IsMirrored"));
         	 					bool bIsMirrored = false;
         	 					IsMirroredProperty->GetValue_InContainer(StructData, &bIsMirrored);
         	 						
         	 					FMapProperty* CurvesProperty = CastField<FMapProperty>(Struct->CustomFindProperty("Curves"));
         	 					// copy Map
         	 					TMap<FName,double> Map = *reinterpret_cast<const TMap<FName, double>*>(static_cast<const uint8*>(StructData) + CurvesProperty->GetOffset_ForInternal());
         	 						
								ValueStruct->InitializeAs(StructPropertyBinding->StructType);

         	 					FChooserPlayerSettings* ChooserPlayerSettings = reinterpret_cast<FChooserPlayerSettings*>(ValueStruct->GetMutableMemory());

         	 					ChooserPlayerSettings->PlaybackRate = 0.0;
         	 					ChooserPlayerSettings->StartTime = Frame / 30.0;
         	 					ChooserPlayerSettings->bMirror = bIsMirrored;

         	 					for (auto& pair : Map)
         	 					{
         	 						ChooserPlayerSettings->CurveOverrides.Values.Add(FAnimCurveOverride{pair.Key, (float)pair.Value});
         	 					}
         	 						
         	 				}
         	 				else
       	 					// END Temporary data conversion codepath
         	 				if (ValueStruct->GetScriptStruct() != StructPropertyBinding->StructType)
         	 				{
         	 					ValueStruct->InitializeAs(StructPropertyBinding->StructType);
								if (PropUtils.IsValid())
								{
									PropUtils->ForceRefresh();
								}
         	 					
         	 				}
	
         	 				BindingHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
							ValueHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
         	 			}

         	 		}
         	 	})
	];
}
	
void FStructOutputDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	static FName ValueName = "Value";
	
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> CurrentProperty = StructPropertyHandle->GetChildHandle(ChildIndex);
		if (CurrentProperty->GetProperty()->GetFName() == ValueName)
		{
			// show the child properties of "Value" as our child properties
			TSharedRef<FInstancedStructDataDetails> DataDetails = MakeShared<FInstancedStructDataDetails>(CurrentProperty);
			ChildBuilder.AddCustomBuilder(DataDetails);
		}
	}
}
	
}


#undef LOCTEXT_NAMESPACE