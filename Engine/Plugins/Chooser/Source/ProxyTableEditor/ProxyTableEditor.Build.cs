// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProxyTableEditor : ModuleRules
	{
		public ProxyTableEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"AssetTools",
					"AssetDefinition",
					"Chooser",
					"ChooserEditor",
					"ProxyTable",
					"UnrealEd",
					"EditorWidgets",
					"ToolWidgets",
					"ToolMenus",
					"SlateCore",
					"Slate",
					"PropertyEditor",
					"InputCore",
					"EditorStyle",
					"PropertyEditor",
					"BlueprintGraph",
					"GraphEditor",
					"GameplayTags",
					"GameplayTagsEditor",
					"StructUtilsEditor",
					"BlendStack"
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}