// Copyright Epic Games, Inc. All Rights Reserved.


#include "EvaluateProxyNode.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "ChooserFunctionLibrary.h"
#include "ProxyTableFunctionLibrary.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeArray.h"
#include "K2Node_MakeStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet/KismetSystemLibrary.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "K2Node_Self.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EvaluateProxyNode)


#define LOCTEXT_NAMESPACE "EvaluateProxyNode"

/////////////////////////////////////////////////////
// UK2Node_EvaluateProxy

UK2Node_EvaluateProxy::UK2Node_EvaluateProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Evaluates an Proxy Table, and returns the resulting Object or Objects.");
}

void UK2Node_EvaluateProxy::UnregisterProxyCallback()
{
	if(CurrentCallbackProxy)
	{
		CurrentCallbackProxy->OnTypeChanged.RemoveAll(this);
		CurrentCallbackProxy = nullptr;
	}
}

void UK2Node_EvaluateProxy::BeginDestroy()
{
	UnregisterProxyCallback();
	
	Super::BeginDestroy();
}

void UK2Node_EvaluateProxy::PostEditUndo()
{
	Super::PostEditUndo();
	ProxyChanged();
}

void UK2Node_EvaluateProxy::DestroyNode()
{
	UnregisterProxyCallback();
	Super::DestroyNode();
}

void UK2Node_EvaluateProxy::ProxyChanged()
{
	if (Proxy != CurrentCallbackProxy)
	{
		UnregisterProxyCallback();
	
		if (Proxy)
		{
			Proxy->OnTypeChanged.AddUObject(this, &UK2Node_EvaluateProxy::TypeChanged);
		}
	
		CurrentCallbackProxy = Proxy;

		AllocateDefaultPins();
	}
}

void UK2Node_EvaluateProxy::TypeChanged(const UClass*)
{
	AllocateDefaultPins();
}


void UK2Node_EvaluateProxy::PreloadRequiredAssets()
{
	if (Proxy)
	{
		Proxy->ConditionalPreload();
	}
    		
	Super::PreloadRequiredAssets();
}

void UK2Node_EvaluateProxy::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	
	UClass* ProxyResultType = UObject::StaticClass();

   	if (Proxy && Proxy->Type)
   	{
   		ProxyResultType = Proxy->Type;
   	}

	if (UEdGraphPin* ResultPin = FindPin(TEXT("Result"), EGPD_Output))
	{
		ResultPin->PinType.PinSubCategoryObject = ProxyResultType;
	}
	else
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, ProxyResultType, TEXT("Result"));
	}
}

FText UK2Node_EvaluateProxy::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EvaluateProxy_Title", "Evaluate Proxy (Deprecated)");
}

FText UK2Node_EvaluateProxy::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	return FText::FromName(Pin->PinName);
}

void UK2Node_EvaluateProxy::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetName() == "Proxy")
	{
		ProxyChanged();
	}
	if (PropertyChangedEvent.Property->GetName() == "Mode")
	{
		AllocateDefaultPins();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_EvaluateProxy::PostLoad()
{
	Super::PostLoad();
	ProxyChanged();
}

void UK2Node_EvaluateProxy::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_EvaluateProxy::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
}

void UK2Node_EvaluateProxy::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);
}

FText UK2Node_EvaluateProxy::GetTooltipText() const
{
	return NodeTooltip;
}

void UK2Node_EvaluateProxy::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_EvaluateProxy::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* ResultPin = FindPinChecked(TEXT("Result"));
	if (ResultPin->HasAnyConnections())
	{
		UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, this);

		CallFunction->SetFromFunction(UProxyTableFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UProxyTableFunctionLibrary, EvaluateProxyAsset)));
		CallFunction->AllocateDefaultPins();

		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(SelfNode, this);
		SelfNode->AllocateDefaultPins();

		SelfNode->FindPin(TEXT("self"))->MakeLinkTo(CallFunction->FindPin(TEXT("ContextObject"))); // add null check + error
		
		UEdGraphPin* ProxyAssetPin = CallFunction->FindPin(TEXT("Proxy"));
		CallFunction->GetSchema()->TrySetDefaultObject(*ProxyAssetPin, Proxy);

		UEdGraphPin* OutputPin = CallFunction->GetReturnValuePin();

		if (Proxy && Proxy->Type)
		{
			UEdGraphPin* OutputClassPin = CallFunction->FindPin(TEXT("ObjectClass"));
			CallFunction->GetSchema()->TrySetDefaultObject(*OutputClassPin, Proxy->Type);
		}

		CompilerContext.MovePinLinksToIntermediate(*ResultPin, *OutputPin);
	}
	
	BreakAllNodeLinks();
}

UK2Node::ERedirectType UK2Node_EvaluateProxy::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_EvaluateProxy::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_EvaluateProxy::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EvaluateProxy::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Animation);
}



//----------------------------------------------------------------------------------------------
// UK2Node_EvaluateProxy2
// New Implementation of EvaluateChooser with support for passing in/out multiple objects and structs

UK2Node_EvaluateProxy2::UK2Node_EvaluateProxy2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("EvaluateChooser2Tooltip", "Evaluates an Chooser Table, and returns the resulting Object or Objects.");
}

void UK2Node_EvaluateProxy2::UnregisterProxyCallback()
{
	if(CurrentCallbackProxy)
	{
		CurrentCallbackProxy->OnTypeChanged.RemoveAll(this);
		CurrentCallbackProxy->OnContextClassChanged.RemoveAll(this);
		CurrentCallbackProxy = nullptr;
	}
}

void UK2Node_EvaluateProxy2::BeginDestroy()
{
	UnregisterProxyCallback();
	
	Super::BeginDestroy();
}

void UK2Node_EvaluateProxy2::PostEditUndo()
{
	Super::PostEditUndo();
	ProxyChanged();
}

void UK2Node_EvaluateProxy2::DestroyNode()
{
	UnregisterProxyCallback();
	Super::DestroyNode();
}


void UK2Node_EvaluateProxy2::ProxyChanged()
{
	if (Proxy != CurrentCallbackProxy)
	{
		UnregisterProxyCallback();
	
		if (Proxy)
		{
			Proxy->OnTypeChanged.AddUObject(this, &UK2Node_EvaluateProxy2::ResultTypeChanged);
			Proxy->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
		}
	
		CurrentCallbackProxy = Proxy;

		ReconstructNode();
	}
}

void UK2Node_EvaluateProxy2::ResultTypeChanged(const UClass*)
{
	AllocateDefaultPins();
}

void UK2Node_EvaluateProxy2::PreloadRequiredAssets()
{
	if (Proxy)
	{
		Proxy->ConditionalPreload();
	}
    		
	Super::PreloadRequiredAssets();
}

void UK2Node_EvaluateProxy2::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	
	UClass* ResultType = UObject::StaticClass();

	if (!FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input))
	{
		// Input - Execution Pin
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	}

	if (!FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Output))
	{
		// Output - Execution Pin
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	}

	if (Proxy)
	{
		// We show input struct contexts pins as "references", because the supplied struct might get modified if it's an In/Out param.
		// We also show output struct contexts pins as references, to show they are there just for convenience and actually pointing to the very same
		// struct that the one on the input side.
		FCreatePinParams StructContextPinCreationParams;
		StructContextPinCreationParams.bIsReference = true;

		for (FInstancedStruct& ContextDataEntry : Proxy->ContextData)
		{
			if (ContextDataEntry.IsValid())
			{
				const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
				if (EntryType == FContextObjectTypeClass::StaticStruct())
				{
					const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
					if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Input))
					{
						Pin->PinType.PinSubCategoryObject = ClassContext.Class;
					}
					else
					{
						CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, ClassContext.Class, ClassContext.Class.GetFName());
					}
				}
				else if (EntryType == FContextObjectTypeStruct::StaticStruct())
				{
					const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();

					if (StructContext.Direction == EContextObjectDirection::Read || StructContext.Direction == EContextObjectDirection::ReadWrite)
					{
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
						{
							Pin->PinType.PinSubCategoryObject = StructContext.Struct;
						}
						else
						{
							CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, StructContext.Struct, StructContext.Struct.GetFName(), StructContextPinCreationParams);
						}
					}

					if (StructContext.Direction == EContextObjectDirection::Write || StructContext.Direction == EContextObjectDirection::ReadWrite)
					{
						const bool bArrayOutput = StructOutputMode == EEvaluateProxyStructOutputMode::Default && Mode == EEvaluateProxyMode::AllResults;
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
						{
							Pin->PinType.PinSubCategoryObject = StructContext.Struct;
							Pin->PinType.ContainerType = bArrayOutput ? EPinContainerType::Array : EPinContainerType::None;
						}
						else
						{
							UEdGraphNode::FCreatePinParams PinParams;
							PinParams.ContainerType = bArrayOutput ? EPinContainerType::Array : EPinContainerType::None;
							PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Struct;
							
							CreatePin(EGPD_Output, PinParams.ValueTerminalType.TerminalCategory, StructContext.Struct, StructContext.Struct.GetFName(), PinParams);
						}
					}

					if (StructContext.Direction == EContextObjectDirection::Read)
					{
					    // for read only structs, remove the output pin if found
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
						{
							RemovePin(Pin);
						}
					}
					
					if (StructContext.Direction == EContextObjectDirection::Write)
					{
					    // for write only structs, remove the input pin if found
						if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
						{
							RemovePin(Pin);
						}
					}
				}
			}
		}
	}

	bool bResultIsClass = false;
   	if (Proxy && Proxy->Type)
   	{
   		ResultType = Proxy->Type;
   		// For class results, in Array mode, we can't easily do the CastToClass - we have to just return an Array of Objects
   		bResultIsClass = Proxy->ResultType == EObjectChooserResultType::ClassResult && Mode != EEvaluateProxyMode::AllResults;
   	}
	
	static const FName ProxyTablePinName = "ProxyTable";
	if (FindPin(ProxyTablePinName, EGPD_Input) == nullptr)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UProxyTable::StaticClass(), ProxyTablePinName);
	}

	if (UEdGraphPin* ResultPin = FindPin(TEXT("Result"), EGPD_Output))
	{
		ResultPin->PinType.PinCategory = bReturnSoftObjectReference ? UEdGraphSchema_K2::PC_SoftObject
								: bResultIsClass ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object;
		ResultPin->PinType.PinSubCategoryObject = ResultType;
		ResultPin->PinType.ContainerType = (Mode == EEvaluateProxyMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
	}
	else
	{
		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.ContainerType = (Mode == EEvaluateProxyMode::AllResults) ? EPinContainerType::Array : EPinContainerType::None;
		PinParams.ValueTerminalType.TerminalCategory = bReturnSoftObjectReference ? UEdGraphSchema_K2::PC_SoftObject
							: bResultIsClass ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object;	
		CreatePin(EGPD_Output, PinParams.ValueTerminalType.TerminalCategory, ResultType, TEXT("Result"), PinParams);
	}
}

FText UK2Node_EvaluateProxy2::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Proxy)
	{
		return FText::Format(LOCTEXT("EvaluateProxy2_TitleWithProxy", "Evaluate Proxy: {0}"), {FText::FromString(Proxy->GetName())});
	}
	else
	{
		return LOCTEXT("EvaluateProxy2_Title", "Evaluate Proxy");
	}
}

FText UK2Node_EvaluateProxy2::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if (Pin->PinName == UEdGraphSchema_K2::PN_Execute)
	{
		return FText();
	}
	return FText::FromName(Pin->PinName);
}

void UK2Node_EvaluateProxy2::PostPlacedNewNode()
{
	// Class default value (Single Struct output mode) is there to not break existing content.
	// for new nodes, set the default to match the object output mode
	StructOutputMode = EEvaluateProxyStructOutputMode::Default;
}

void UK2Node_EvaluateProxy2::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FString PropertyName = PropertyChangedEvent.Property->GetName();
	
	if (PropertyName == "Proxy")
	{
		ProxyChanged();
	}
	if (PropertyName == "Mode" || PropertyName == "StructOutputMode" || PropertyName == "bReturnSoftObjectReference")
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_EvaluateProxy2::PostLoad()
{
	Super::PostLoad();
	
	if (Proxy)
	{
		Proxy->OnTypeChanged.AddUObject(this, &UK2Node_EvaluateProxy2::ResultTypeChanged);
		Proxy->OnContextClassChanged.AddUObject(this, &UK2Node::ReconstructNode);
	}

	CurrentCallbackProxy = Proxy;
}

void UK2Node_EvaluateProxy2::PinConnectionListChanged(UEdGraphPin* Pin)
{

	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_EvaluateProxy2::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
}

void UK2Node_EvaluateProxy2::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);
}

FText UK2Node_EvaluateProxy2::GetTooltipText() const
{
	return NodeTooltip;
}

void UK2Node_EvaluateProxy2::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_EvaluateProxy2::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* ExecInput = GetExecPin();
	UEdGraphPin* ExecOutput = GetPassThroughPin(ExecInput);
	UEdGraphPin* ResultPin = FindPinChecked(TEXT("Result"));

	UEdGraphPin* PreviousNodeExecOutput = nullptr;
	
	if (ExecInput->HasAnyConnections())
	{
		UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, this);
		
		if (bReturnSoftObjectReference)
		{
			if (Mode == EEvaluateProxyMode::AllResults)
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseMultiSoft)));
			}
			else
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseSoft)));
			}
		}
		else
		{
			if (Mode == EEvaluateProxyMode::AllResults)
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBaseMulti)));
			}
			else
			{
				CallFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, EvaluateObjectChooserBase)));
			}
		}
		CallFunction->AllocateDefaultPins();
		
		UK2Node_CallFunction* ContextStructNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		ContextStructNode->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, MakeChooserEvaluationContext)));
        CompilerContext.MessageLog.NotifyIntermediateObjectCreation(ContextStructNode, this);
        ContextStructNode->AllocateDefaultPins();
		CompilerContext.MovePinLinksToIntermediate(*ExecInput, *ContextStructNode->GetExecPin());
		UEdGraphPin* ContextStructPin = ContextStructNode->GetReturnValuePin();

		ContextStructPin->MakeLinkTo(CallFunction->FindPin(FName("Context")));
		PreviousNodeExecOutput = ContextStructNode->GetThenPin();

		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(SelfNode, this);
		SelfNode->AllocateDefaultPins();
		UEdGraphPin* SelfPin = SelfNode->FindPin(TEXT("self"));
		
		int OutputStructArrayIndex = 0;
		
		if (Proxy)
		{
			bool bFoundObject = false;
			int ContextDataCount = Proxy->ContextData.Num();
			for(int ContextDataIndex = 0; ContextDataIndex < ContextDataCount; ContextDataIndex++)
			{
				FInstancedStruct& ContextDataEntry = Proxy->ContextData[ContextDataIndex];
				if (ContextDataEntry.IsValid())
				{
					const UScriptStruct* EntryType = ContextDataEntry.GetScriptStruct();
					if (EntryType == FContextObjectTypeClass::StaticStruct())
					{
						bFoundObject = true;
						const FContextObjectTypeClass& ClassContext = ContextDataEntry.Get<FContextObjectTypeClass>();
						if (ClassContext.Class)
						{
							if (UEdGraphPin* Pin = FindPin(ClassContext.Class.GetFName(), EGPD_Input))
							{
								UK2Node_CallFunction* AddObjectFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
								CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddObjectFunction, this);
								AddObjectFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserObjectInput)));
								AddObjectFunction->AllocateDefaultPins();
								ContextStructPin->MakeLinkTo(AddObjectFunction->FindPin(FName("Context")));

								PreviousNodeExecOutput->MakeLinkTo(AddObjectFunction->GetExecPin());
								PreviousNodeExecOutput = AddObjectFunction->GetThenPin();
								
								UEdGraphPin* AddObjectPin = AddObjectFunction->FindPin(FName("Object"));
								
								if (Pin->HasAnyConnections())
								{
									CompilerContext.MovePinLinksToIntermediate(*Pin, *AddObjectPin);
								}
								else // would be nice to check that self is the same type as this Object parameter
								{
									// auto connect self node to any disconnected object pin
									SelfPin->MakeLinkTo(AddObjectPin);
								}
							}
						}
					}
					else if (EntryType == FContextObjectTypeStruct::StaticStruct())
					{
						const FContextObjectTypeStruct& StructContext = ContextDataEntry.Get<FContextObjectTypeStruct>();
						if (StructContext.Struct)
						{
							UK2Node_CallFunction* AddStructFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
							CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddStructFunction, this);
							
							if (StructContext.Direction == EContextObjectDirection::Read || Mode == EEvaluateProxyMode::FirstResult || StructOutputMode == EEvaluateProxyStructOutputMode::SingleOutputStruct)
							{
								AddStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserStructInput)));
							}
							else
							{
								AddStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserStructInputOutput)));
							}
							
							AddStructFunction->AllocateDefaultPins();
							ContextStructPin->MakeLinkTo(AddStructFunction->FindPin(FName("Context")));

							PreviousNodeExecOutput->MakeLinkTo(AddStructFunction->GetExecPin());
							PreviousNodeExecOutput = AddStructFunction->GetThenPin();
							
							UEdGraphPin* AddStructPin = AddStructFunction->FindPin(FName("Value"));
							// not sure why this isn't automatically happening:
							AddStructPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
							AddStructPin->PinType.PinSubCategoryObject = StructContext.Struct;

							bool bLinked = false;
							if (StructContext.Direction != EContextObjectDirection::Write)
							{
								if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Input))
								{
									if (Pin->HasAnyConnections())
									{
										CompilerContext.MovePinLinksToIntermediate(*Pin, *AddStructPin);
										bLinked = true;
									}
								}
							}
						
							if (!bLinked)
							{
								// create a struct to hold the output (or for input structs that were not connected to anything)
								UK2Node_MakeStruct* OutputStructNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeStruct>(this, SourceGraph);
								CompilerContext.MessageLog.NotifyIntermediateObjectCreation(OutputStructNode, this);
								OutputStructNode->PostPlacedNewNode();
								OutputStructNode->StructType = static_cast<UScriptStruct*>(StructContext.Struct);
								OutputStructNode->AllocateDefaultPins();
								OutputStructNode->FindPin(StructContext.Struct->GetFName(), EGPD_Output)->MakeLinkTo(AddStructPin);
							}

							if (StructContext.Direction != EContextObjectDirection::Read)
							{
								if (UEdGraphPin* Pin = FindPin(StructContext.Struct.GetFName(), EGPD_Output))
								{
									if (Pin->HasAnyConnections())
									{
										// set up struct output pin
										UK2Node_CallFunction* GetStructFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
										CompilerContext.MessageLog.NotifyIntermediateObjectCreation(GetStructFunction, this);
										
										if (Mode == EEvaluateProxyMode::FirstResult || StructOutputMode == EEvaluateProxyStructOutputMode::SingleOutputStruct)
										{
											GetStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserStructOutput)));
											GetStructFunction->AllocateDefaultPins();
											GetStructFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(ContextDataIndex);
										}
										else
										{
											GetStructFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, GetChooserStructOutputArray)));
											GetStructFunction->AllocateDefaultPins();
											GetStructFunction->FindPin(FName("Index"))->DefaultValue = FString::FromInt(OutputStructArrayIndex);
										}

										ContextStructPin->MakeLinkTo(GetStructFunction->FindPin(FName("Context")));
										UEdGraphPin* ValuePin = GetStructFunction->FindPin(FName("Value"));
										// not sure why this isn't automatically happening:
										ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
										ValuePin->PinType.PinSubCategoryObject = StructContext.Struct;
					
										CompilerContext.MovePinLinksToIntermediate(*Pin, *ValuePin);
									}
								}
								OutputStructArrayIndex ++;
							}
						}
					}
				}
			}
			
			if (!bFoundObject)
			{
				// add Self reference to the end of the context, for debugging purposes
				UK2Node_CallFunction* AddObjectFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AddObjectFunction, this);
				AddObjectFunction->SetFromFunction(UChooserFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UChooserFunctionLibrary, AddChooserObjectInput)));
				AddObjectFunction->AllocateDefaultPins();
				ContextStructPin->MakeLinkTo(AddObjectFunction->FindPin(FName("Context")));
	
				PreviousNodeExecOutput->MakeLinkTo(AddObjectFunction->GetExecPin());
				PreviousNodeExecOutput = AddObjectFunction->GetThenPin();
									
				UEdGraphPin* AddObjectPin = AddObjectFunction->FindPin(FName("Object"));
				SelfPin->MakeLinkTo(AddObjectPin);
			}
			

			if (PreviousNodeExecOutput)
			{
				PreviousNodeExecOutput->MakeLinkTo(CallFunction->GetExecPin());
			}
			else
			{
				CompilerContext.MovePinLinksToIntermediate(*ExecInput, *CallFunction->GetExecPin());
			}
			
			CompilerContext.MovePinLinksToIntermediate(*ExecOutput, *CallFunction->GetThenPin());
		}
		
		UK2Node_CallFunction* MakeLookupProxyFunction = nullptr;

		UEdGraphPin* OverrideTablePin = FindPinChecked(TEXT("ProxyTable"));
		if (OverrideTablePin->HasAnyConnections() || OverrideTablePin->DefaultObject)
		{
			MakeLookupProxyFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
   			MakeLookupProxyFunction->SetFromFunction(UProxyTableFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UProxyTableFunctionLibrary, MakeLookupProxyWithOverrideTable)));
       		MakeLookupProxyFunction->AllocateDefaultPins();

			UEdGraphPin* TargetOverrideTablePin = MakeLookupProxyFunction->FindPinChecked(TEXT("ProxyTable"));
			CompilerContext.MovePinLinksToIntermediate(*OverrideTablePin, *TargetOverrideTablePin);
		}
		else
		{
			MakeLookupProxyFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			MakeLookupProxyFunction->SetFromFunction(UProxyTableFunctionLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UProxyTableFunctionLibrary, MakeLookupProxy)));
			MakeLookupProxyFunction->AllocateDefaultPins();
		}
		
   		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeLookupProxyFunction, this);
		
		UEdGraphPin* ProxyPin = MakeLookupProxyFunction->FindPin(FName("Proxy"));
		MakeLookupProxyFunction->GetSchema()->TrySetDefaultObject(*ProxyPin, Proxy);

		MakeLookupProxyFunction->GetReturnValuePin()->MakeLinkTo(CallFunction->FindPin(FName("ObjectChooser")));

		UEdGraphPin* OutputPin = CallFunction->GetReturnValuePin();

		if (Proxy) 
		{
			if (Proxy->Type)
			{
				if (UEdGraphPin* OutputClassPin = CallFunction->FindPin(TEXT("ObjectClass")))
				{
					CallFunction->GetSchema()->TrySetDefaultObject(*OutputClassPin, Proxy->Type);
				}
			}

			if (UEdGraphPin* ResultIsClassPin = CallFunction->FindPin(TEXT("bResultIsClass")))
			{
				// this ensures that the function does the right kind of type validation
				CallFunction->GetSchema()->TrySetDefaultValue(*ResultIsClassPin, Proxy->ResultType == EObjectChooserResultType::ClassResult ? TEXT("true") : TEXT("false"));

				// For class results, in Array mode, we can't easily do the CastToClass - we have to just return an Array of Objects
				if (Proxy->ResultType == EObjectChooserResultType::ClassResult && Mode != EEvaluateProxyMode::AllResults)
				{
					// if it's a class we need to add a CastToClass function to cast it from an Object pointer to a Class pointer
					UK2Node_CallFunction* CastToClass = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
					CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CastToClass, this);

					CastToClass->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, Conv_ObjectToClass)));
					CastToClass->AllocateDefaultPins();

					if (UEdGraphPin* ClassPin = CastToClass->FindPin(TEXT("Class")))
					{
						CastToClass->GetSchema()->TrySetDefaultObject(*ClassPin, Proxy->Type);
					}

					if (UEdGraphPin* ObjectPin = CastToClass->FindPin(FName("Object")))
					{
						CallFunction->GetReturnValuePin()->MakeLinkTo(ObjectPin);
					}

					OutputPin = CastToClass->GetReturnValuePin();
				}
			}
		}

		CompilerContext.MovePinLinksToIntermediate(*ResultPin, *OutputPin);
	}
	
	BreakAllNodeLinks();
}

UK2Node::ERedirectType UK2Node_EvaluateProxy2::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_EvaluateProxy2::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_EvaluateProxy2::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EvaluateProxy2::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Animation);
}



#undef LOCTEXT_NAMESPACE
