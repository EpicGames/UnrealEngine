// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "ProxyTable.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EvaluateProxyNode.generated.h"

UENUM()
enum class EEvaluateProxyMode
{
	FirstResult,
	AllResults
};

UENUM()
enum class EEvaluateProxyStructOutputMode
{
	Default,
	SingleOutputStruct,
};

UCLASS(MinimalAPI, Hidden)
class UK2Node_EvaluateProxy : public UK2Node
{
	GENERATED_UCLASS_BODY()
	virtual void BeginDestroy() override;

	//~ Begin UObject Interface
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool IsNodePure() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual void PreloadRequiredAssets() override;
   
	virtual void DestroyNode() override; 
	//~ End UK2Node Interface.

private:

	void ProxyChanged();
	void TypeChanged(const UClass*);
	void UnregisterProxyCallback();
	UProxyAsset* CurrentCallbackProxy = nullptr;


	UPROPERTY(EditAnywhere, Category = "Proxy")
	TObjectPtr<UProxyAsset> Proxy;

	/** Tooltip text for this node. */
	FText NodeTooltip;
};

//----------------------------------------------------------------------------------------------
// UK2Node_EvaluateChooser2
// New Implementation of EvaluateChooser with support for passing in/out multiple objects and structs
UCLASS(MinimalAPI)
class UK2Node_EvaluateProxy2 : public UK2Node
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool IsNodePure() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual void PreloadRequiredAssets() override;

	virtual void DestroyNode() override;
    
	//~ End UK2Node Interface.

private:
	void ProxyChanged();
	void ResultTypeChanged(const UClass*);
	void UnregisterProxyCallback();
	UProxyAsset* CurrentCallbackProxy = nullptr;

	UPROPERTY(EditAnywhere, Category = "Proxy")
	TObjectPtr<UProxyAsset> Proxy;

	UPROPERTY(EditAnywhere, Category = "Proxy")
	EEvaluateProxyMode Mode;

	// This property exist for backwards compatibility with old content (where only single output struct was supported).
	// New content should use "Default" setting to stay in sync with the regular output mode.
	UPROPERTY(EditAnywhere, Category = "Proxy")
	EEvaluateProxyStructOutputMode StructOutputMode = EEvaluateProxyStructOutputMode::SingleOutputStruct;
	
	UPROPERTY(EditAnywhere, Category = "Proxy")
	bool bReturnSoftObjectReference = false;

	/** Tooltip text for this node. */
	FText NodeTooltip;
};
