// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyTableUncookedModule.h"
#include "Modules/ModuleManager.h"

namespace UE::ProxyTable
{

void FProxyTableUncookedModule::StartupModule()
{
}

void FProxyTableUncookedModule::ShutdownModule()
{
}

}

IMPLEMENT_MODULE(UE::ProxyTable::FProxyTableUncookedModule, ProxyTableUncooked);