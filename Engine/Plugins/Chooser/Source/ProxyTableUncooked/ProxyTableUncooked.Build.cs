// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ProxyTableUncooked : ModuleRules
	{
		public ProxyTableUncooked(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[] {"ProxyTable"});
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Chooser",
					"Engine",
					"AssetTools",
					"UnrealEd",
					"AssetDefinition",
					"GameplayTags",
					"BlueprintGraph",
					"KismetCompiler"
					// ... add private dependencies that you statically link with here ...
				}
			);
		}
	}
}
